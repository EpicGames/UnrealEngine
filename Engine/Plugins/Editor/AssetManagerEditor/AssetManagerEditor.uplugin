{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Asset Manager Editor",
	"Description" : "Editor UI and utilities for managing and auditing Assets on disk",
	"Category" : "Editor",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : true,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	
	"Modules" :
	[
		{
			"Name" : "AssetManagerEditor",
			"Type": "UncookedOnly" ,
			"LoadingPhase" : "PreDefault"
		}
	],
	"Plugins": [
		{
			"Name": "ContentBrowserAssetDataSource",
			"Enabled": true
		}
	]
}