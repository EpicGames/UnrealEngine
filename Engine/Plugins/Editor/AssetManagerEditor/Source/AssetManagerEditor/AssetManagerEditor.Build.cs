// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class AssetManagerEditor : ModuleRules
{
	public AssetManagerEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"StatsViewer",
			}
		);

		PublicDependencyModuleNames.AddRange(
			new string[] { 
				"Core",
				"CoreUObject",
				"Engine",
				"TargetPlatform",
			}
		);
		
		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"CookMetadata",
				"Slate",
				"SlateCore",
				"ApplicationCore",
				"InputCore",
				"EditorFramework",
				"UnrealEd",
				"AssetRegistry",
				"Json",
				"CollectionManager",
				"ContentBrowser",
				"ContentBrowserData",
				"ContentBrowserAssetDataSource",
				"WorkspaceMenuStructure",
				"AssetDefinition",
				"AssetTools",
				"PropertyEditor",
				"GraphEditor",
				"BlueprintGraph",
				"KismetCompiler",
				"LevelEditor",
				"SandboxFile",
				"EditorWidgets",
				"TreeMap",
				"ToolMenus",
				"ToolWidgets",
				"TraceInsightsCore",
				"SourceControl",
				"SourceControlWindows",
				"UncontrolledChangelists",
				"Projects",
				"StatusBar",
			}
		);

		// This module needs AutoRTFM so disable the auto disable since its in a plugin.
		// The delegate `PackageMarkedDirtyEvent` on `UPackage` is being used.
		bDisableAutoRTFMInstrumentation = false;
	}
}
