// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetManagerEditorCommands.h"
#include "Styling/AppStyle.h"

//////////////////////////////////////////////////////////////////////////
// FAssetManagerEditorCommands

#define LOCTEXT_NAMESPACE "AssetManagerEditorCommands"

FAssetManagerEditorCommands::FAssetManagerEditorCommands() : TCommands<FAssetManagerEditorCommands>(
	"AssetManagerEditorCommands",
	NSLOCTEXT("Contexts", "AssetManagerEditorCommands", "Asset Management"),
	NAME_None, 
	FAppStyle::GetAppStyleSetName())
{
}

void FAssetManagerEditorCommands::RegisterCommands()
{
	UI_COMMAND(CopyAssetReferences, "Copy References", "Copy the selected assets' references to the clipboard", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CopyAssetPaths, "Copy Paths", "Copy the selected assets' file paths to the clipboard", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ViewReferences, "Reference Viewer...", "Launches the reference viewer showing the selected assets' references", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift | EModifierKey::Alt, EKeys::R));
	UI_COMMAND(ViewSizeMap, "Size Map...", "Displays an interactive map showing the approximate size of this asset and everything it references", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift | EModifierKey::Alt, EKeys::M));
	UI_COMMAND(ViewShaderCookStatistics, "Shader Cook Statistics...", "Show Shader CookStatistics", EUserInterfaceActionType::Button, FInputChord());
	
	UI_COMMAND(ViewAssetAudit, "Audit Assets...", "Opens the Asset Audit UI and displays information about the selected assets", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift | EModifierKey::Alt, EKeys::A));

	UI_COMMAND(OpenSelectedInAssetEditor, "Edit...", "Opens the selected asset in the relevant editor.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::E));
	UI_COMMAND(ZoomToFit, "Zoom to Fit", "Zoom in and center the view on the selected item", EUserInterfaceActionType::Button, FInputChord(EKeys::F));
	UI_COMMAND(ResolveReferencingProperties, "Resolve Referencing Properties", "Resolve and display the property (or properties) referencing this asset", EUserInterfaceActionType::Button, FInputChord());

	UI_COMMAND(IncreaseReferencerSearchDepth, "Increase Referencer Search Depth", "Increase the Referencer Search Depth", EUserInterfaceActionType::Button, FInputChord(EKeys::R));
	UI_COMMAND(DecreaseReferencerSearchDepth, "Decrease Referencer Search Depth", "Decrease the Referencer Search Depth", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::R));
	UI_COMMAND(SetReferencerSearchDepth, "Set Referencer Search Depth", "Set the Referencer Search Depth", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::R));

	UI_COMMAND(IncreaseDependencySearchDepth, "Increase Dependency Search Depth", "Increase the Dependency Search Depth", EUserInterfaceActionType::Button, FInputChord(EKeys::D));
	UI_COMMAND(DecreaseDependencySearchDepth, "Decrease Dependency Search Depth", "Decrease the Dependency Search Depth", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::D));
	UI_COMMAND(SetDependencySearchDepth, "Set Dependency Search Depth", "Set the Dependency Search Depth", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::D));

	UI_COMMAND(IncreaseBreadth, "Increase Search Breadth", "Increase the Breadth Limit", EUserInterfaceActionType::Button, FInputChord( EKeys::L));
	UI_COMMAND(DecreaseBreadth, "Decrease Search Breadth", "Decrease the Breadth Limit", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::L));
	UI_COMMAND(SetBreadth, "Set the Breadth", "Set the Breadth Limit", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::L));

	UI_COMMAND(ShowSoftReferences, "Show Soft References", "Toggles visibility of Soft References", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::S));
	UI_COMMAND(ShowHardReferences, "Show Hard References", "Toggles visibility of Hard References", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::H));
	UI_COMMAND(EditorOnlyReferenceFilterTypeGame, "Show Game References",
		"Only UsedInGame references are displayed. UsedInGame references cause the dependencies to be cooked and"
		" cause management assignments from the referencer to be propagated to the dependencies.",
		EUserInterfaceActionType::RadioButton, FInputChord(EKeys::G));
	UI_COMMAND(EditorOnlyReferenceFilterTypePropagation, "Show Propagation References",
		"UsedInGame and Build references are displayed. Build references are displayed even if they are EditorOnly."
		" All of these references cause management assignments from the referencer to be propagated to the"
		" dependencies, but the EditorOnly Build dependencies do not cause the dependencies to be cooked.",
		EUserInterfaceActionType::RadioButton, FInputChord(EKeys::B));
	UI_COMMAND(EditorOnlyReferenceFilterTypeEditorOnly, "Show EditorOnly References",
		"UsedInGame, Build, and EditorOnly references are displayed. EditorOnly references do not cause the"
		" dependencies to be cooked and do not propagate management assignments. Use this mode to see how assets"
		" interact in the editor.",
		EUserInterfaceActionType::RadioButton, FInputChord(EKeys::E));
	UI_COMMAND(ShowManagementReferences, "Show Management References","Toggles visibility of Management References (i.e. PrimaryAssetIDs)", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::M));
	UI_COMMAND(ShowNameReferences, "Show Name References","Toggles visibility of Name References (i.e. Gameplay Tags and Data Table Row Handles)", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::N));
	UI_COMMAND(ShowCodePackages, "Show C++ Packages","Toggles visibility of C++ Packages", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::C));

	UI_COMMAND(ShowDuplicates, "Show Duplicate References", "Toggles visibility of Duplicate References", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::U));
	UI_COMMAND(CompactMode, "Compact Mode", "Toggles Compact View", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::V));
	UI_COMMAND(ShowExternalReferencers, "Show External Referencers", "Toggles Show External Referencers", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::X));
	UI_COMMAND(FilterSearch, "Filter Search Results", "Toggles filtering of search results", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(ShowCommentPath, "Show Asset Path", "Toggles visibility of the Asset Path shown as a comment", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::P));
	UI_COMMAND(Filters, "Toggle the AssetType Filters", "Toggles the AssetType Filters", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::A));
	UI_COMMAND(AutoFilters, "Auto Populate Filters", "Toggles autopopulating of AssetType Filters", EUserInterfaceActionType::ToggleButton, FInputChord());
	UI_COMMAND(FindPath, "Enable Find Path Tool", "Toggle the Find Path Tool", EUserInterfaceActionType::ToggleButton, FInputChord());

	UI_COMMAND(CopyPaths, "Copy the Asset Path", "Copies the Asset Path to the Clipboard", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::C));

	UI_COMMAND(ReCenterGraph, "Re-Center Graph", "Re-centers the graph on this node, showing all referencers and references for this asset instead", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CopyReferencedObjects, "Copy Referenced Objects List", "Copies the list of objects that the selected asset references to the clipboard.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(CopyReferencingObjects, "Copy Referencing Objects List", "Copies the list of objects that reference the selected asset to the clipboard.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowReferencedObjects, "Show Referenced Objects List", "Shows a list of objects that the selected asset references.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowReferencingObjects, "Show Referencing Objects List", "Shows a list of objects that reference the selected asset.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(ShowReferenceTree, "Show Reference Tree", "Shows a reference tree for the selected asset.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(AuditReferencedObjects, "Audit References", "Opens the Asset Audit UI and displays information about all assets referenced by selected asset", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(Find, "Find", "Find objects in the reference viewer.", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::F));

	UI_COMMAND(MakeLocalCollectionWithReferencers, "Local", "Local. This collection is only visible to you and is not in revision control.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(MakePrivateCollectionWithReferencers, "Private", "Private. This collection is only visible to you.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(MakeSharedCollectionWithReferencers, "Shared", "Shared. This collection is visible to everyone.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(MakeLocalCollectionWithDependencies, "Local", "Local. This collection is only visible to you and is not in revision control.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(MakePrivateCollectionWithDependencies, "Private", "Private. This collection is only visible to you.", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(MakeSharedCollectionWithDependencies, "Shared", "Shared. This collection is visible to everyone.", EUserInterfaceActionType::Button, FInputChord());
}

#undef LOCTEXT_NAMESPACE
