// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

// Actions that can be invoked in the reference viewer
class FAssetManagerEditorCommands : public TCommands<FAssetManagerEditorCommands>
{
public:
	FAssetManagerEditorCommands();

	// TCommands<> interface
	virtual void RegisterCommands() override;
	// End of TCommands<> interface

	// Copy asset references to the clipboard
	TSharedPtr<FUICommandInfo> CopyAssetReferences;

	// Copy asset file paths to the clipboard
	TSharedPtr<FUICommandInfo> CopyAssetPaths;

	// Shows the reference viewer for the selected assets
	TSharedPtr<FUICommandInfo> ViewReferences;

	// Shows a size map for the selected assets
	TSharedPtr<FUICommandInfo> ViewSizeMap;

	// Shows shader cook statistics
	TSharedPtr<FUICommandInfo> ViewShaderCookStatistics;

	// Adds assets to asset audit window
	TSharedPtr<FUICommandInfo> ViewAssetAudit;

	// Opens the selected asset in the asset editor
	TSharedPtr<FUICommandInfo> OpenSelectedInAssetEditor;

	// Re-constructs the graph with the selected asset as the center
	TSharedPtr<FUICommandInfo> ReCenterGraph;

	// Increase Referencer Search Depth
	TSharedPtr<FUICommandInfo> IncreaseReferencerSearchDepth;

	// Decrease Referencer Search Depth
	TSharedPtr<FUICommandInfo> DecreaseReferencerSearchDepth;

	// Set the Referencer Serach Depth 
	TSharedPtr<FUICommandInfo> SetReferencerSearchDepth;

	// Increase Dependency Search Depth
	TSharedPtr<FUICommandInfo> IncreaseDependencySearchDepth;

	// Decrease Dependency Search Depth
	TSharedPtr<FUICommandInfo> DecreaseDependencySearchDepth;

	// Set the Dependency Serach Depth 
	TSharedPtr<FUICommandInfo> SetDependencySearchDepth;

	// Increase Search Breadth
	TSharedPtr<FUICommandInfo> IncreaseBreadth;

	// Decrease Search Breadth
	TSharedPtr<FUICommandInfo> DecreaseBreadth;

	// Set the Breadth LImit
	TSharedPtr<FUICommandInfo> SetBreadth;

	// Toggles visiblity of Soft References
	TSharedPtr<FUICommandInfo> ShowSoftReferences;

	// Toggles visiblity of Hard References
	TSharedPtr<FUICommandInfo> ShowHardReferences;

	// Sets which level of filter type between Game and EditorOnly is visible
	TSharedPtr<FUICommandInfo> EditorOnlyReferenceFilterTypeGame;
	TSharedPtr<FUICommandInfo> EditorOnlyReferenceFilterTypePropagation;
	TSharedPtr<FUICommandInfo> EditorOnlyReferenceFilterTypeEditorOnly;

	// Toggles visiblity of Management References (i.e. PrimaryAssetIDs)
	TSharedPtr<FUICommandInfo> ShowManagementReferences;

	// Toggles visiblity of Name References (i.e. Gameplay Tags and Data Table Row Handles)
	TSharedPtr<FUICommandInfo> ShowNameReferences;

	// Toggles visiblity of Native Packages
	TSharedPtr<FUICommandInfo> ShowCodePackages;

	// Toggles visiblity of Duplicate References
	TSharedPtr<FUICommandInfo> ShowDuplicates;

	// Toggles Compact Mode
	TSharedPtr<FUICommandInfo> CompactMode;

	// Toggles Show External Referencers
	TSharedPtr<FUICommandInfo> ShowExternalReferencers;

	// Toggles Path Comment 
	TSharedPtr<FUICommandInfo> ShowCommentPath;

	// Toggles FilterBar Filters On/Off
	TSharedPtr<FUICommandInfo> Filters;

	// Toggles if Asset Type Filters are Auto Populated & Updated
	TSharedPtr<FUICommandInfo> AutoFilters;

	// Copies the selected Asset Paths to the Clipboard
	TSharedPtr<FUICommandInfo> CopyPaths;

	// Toggles whether search results are filtered or just selected 
	TSharedPtr<FUICommandInfo> FilterSearch;

	// Toggles the Find Path Mode
	TSharedPtr<FUICommandInfo> FindPath;

	// Copies the list of objects that the selected asset references
	TSharedPtr<FUICommandInfo> CopyReferencedObjects;

	// Copies the list of objects that reference the selected asset
	TSharedPtr<FUICommandInfo> CopyReferencingObjects;

	// Shows a list of objects that the selected asset references
	TSharedPtr<FUICommandInfo> ShowReferencedObjects;

	// Shows a list of objects that reference the selected asset
	TSharedPtr<FUICommandInfo> ShowReferencingObjects;

	// Shows a reference tree for the selected asset
	TSharedPtr<FUICommandInfo> ShowReferenceTree;

	// Adds all referenced objects to asset audit window
	TSharedPtr<FUICommandInfo> AuditReferencedObjects;

	/** Zoom in to fit the selected objects in the window */
	TSharedPtr<FUICommandInfo> ZoomToFit;

	/** Display the list of properties which are referencing the selected node from its referencing node(s) */
	TSharedPtr<FUICommandInfo> ResolveReferencingProperties;

	/** Start finding objects */
	TSharedPtr<FUICommandInfo> Find;

	// Creates a new collection with the list of assets that this asset references, user selects which ECollectionShareType to use.
	TSharedPtr<FUICommandInfo> MakeLocalCollectionWithReferencers;
	TSharedPtr<FUICommandInfo> MakePrivateCollectionWithReferencers;
	TSharedPtr<FUICommandInfo> MakeSharedCollectionWithReferencers;
	TSharedPtr<FUICommandInfo> MakeLocalCollectionWithDependencies;
	TSharedPtr<FUICommandInfo> MakePrivateCollectionWithDependencies;
	TSharedPtr<FUICommandInfo> MakeSharedCollectionWithDependencies;
};
