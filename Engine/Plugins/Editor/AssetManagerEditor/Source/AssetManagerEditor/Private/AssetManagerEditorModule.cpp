// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetManagerEditorModule.h"

#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ContentBrowserDataLegacyBridge.h"
#include "HAL/PlatformFile.h"
#include "PrimaryAssetTypeCustomization.h"
#include "IDesktopPlatform.h"
#include "PrimaryAssetIdCustomization.h"
#include "Misc/CommandLine.h"
#include "SAssetAuditBrowser.h"

#include "CollectionManagerModule.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "AssetRegistry/ARFilter.h"
#include "Misc/FileHelper.h"
#include "Misc/PackageName.h"
#include "ProfilingDebugging/ProfilingHelpers.h"
#include "Engine/AssetManager.h"
#include "ContentBrowserModule.h"
#include "ReferenceViewer/HistoryManager.h"
#include "WorkspaceMenuStructure.h"
#include "ToolMenu.h"
#include "WorkspaceMenuStructureModule.h"
#include "Framework/Application/SlateApplication.h"
#include "ToolMenuEntry.h"
#include "Widgets/Docking/SDockTab.h"
#include "ToolMenuSection.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SToolTip.h"
#include "PropertyCustomizationHelpers.h"

#include "LevelEditor.h"
#include "GraphEditorModule.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "IPlatformFileSandboxWrapper.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformFileManager.h"
#include "Serialization/ArrayReader.h"
#include "EdGraphUtilities.h"
#include "EdGraphSchema_K2.h"
#include "AssetManagerEditorCommands.h"
#include "AssetSourceControlContextMenu.h"
#include "AssetToolsModule.h"
#include "ReferenceViewer/SReferenceViewer.h"
#include "ReferenceViewer/SReferenceNode.h"
#include "ReferenceViewer/SReferencedPropertiesNode.h"
#include "ReferenceViewer/EdGraphNode_Reference.h"
#include "ReferenceViewer/EdGraphNode_ReferencedProperties.h"
#include "SSizeMap.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Framework/Docking/TabManager.h"
#include "Framework/Notifications/NotificationManager.h"
#include "DesktopPlatformModule.h"
#include "Misc/FileHelper.h"
#include "Misc/MessageDialog.h"
#include "Misc/ScopedSlowTask.h"
#include "IStatsViewer.h"
#include "StatsViewerModule.h"
#include "ToolMenus.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "ContentBrowserMenuContexts.h"
#include "IContentBrowserDataModule.h"
#include "ContentBrowserDataSubsystem.h"
#include "TreeView/AssetTable.h"
#include "TreeView/SAssetTableTreeView.h"
#include "ISourceControlModule.h"

#define LOCTEXT_NAMESPACE "AssetManagerEditor"

DEFINE_LOG_CATEGORY(LogAssetManagerEditor);

namespace UE::AssetManagerEditor::Private
{
	bool bUseMultipleReferenceViewerTabs = true;
	static FAutoConsoleVariableRef CVarUseMultipleReferenceViewerTabs(
		TEXT("AssetManagerEditor.OpenReferenceViewerInNewTab"),
		bUseMultipleReferenceViewerTabs,
		TEXT("Whether to use multiple Reference Viewer Tabs (one per asset selection) or not.")
	);

	static FName ReferenceViewerLabel(TEXT("Reference Viewer"));
}

class FAssetManagerGraphPanelNodeFactory : public FGraphPanelNodeFactory
{
	virtual TSharedPtr<SGraphNode> CreateNode(UEdGraphNode* Node) const override
	{
		if (UEdGraphNode_Reference* DependencyNode = Cast<UEdGraphNode_Reference>(Node))
		{
			return SNew(SReferenceNode, DependencyNode);
		}
		else if (UEdGraphNode_ReferencedProperties* PropertiesNode = Cast<UEdGraphNode_ReferencedProperties>(Node))
		{
			return SNew(SReferencedPropertiesNode, PropertiesNode);
		}

		return nullptr;
	}
};

class FAssetManagerGraphPanelPinFactory : public FGraphPanelPinFactory
{
	virtual TSharedPtr<SGraphPin> CreatePin(UEdGraphPin* InPin) const override
	{
		if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && InPin->PinType.PinSubCategoryObject == TBaseStructure<FPrimaryAssetId>::Get())
		{
			return SNew(SPrimaryAssetIdGraphPin, InPin);
		}
		if (InPin->PinType.PinCategory == UEdGraphSchema_K2::PC_Struct && InPin->PinType.PinSubCategoryObject == TBaseStructure<FPrimaryAssetType>::Get())
		{
			return SNew(SPrimaryAssetTypeGraphPin, InPin);
		}

		return nullptr;
	}
};

const FName IAssetManagerEditorModule::ChunkFakeAssetDataPackageName = FName("/Temp/PrimaryAsset/PackageChunk");
const FName IAssetManagerEditorModule::PrimaryAssetFakeAssetDataPackagePath = FName("/Temp/PrimaryAsset");
const FPrimaryAssetType IAssetManagerEditorModule::AllPrimaryAssetTypes = FName("AllTypes");

const FName IAssetManagerEditorModule::ResourceSizeName = FName("ResourceSize");
const FName IAssetManagerEditorModule::DiskSizeName = FName("DiskSize");
const FName IAssetManagerEditorModule::ManagedResourceSizeName = FName("ManagedResourceSize");
const FName IAssetManagerEditorModule::ManagedDiskSizeName = FName("ManagedDiskSize");
const FName IAssetManagerEditorModule::TotalUsageName = FName("TotalUsage");
const FName IAssetManagerEditorModule::CookRuleName = FName("CookRule");
const FName IAssetManagerEditorModule::ChunksName = FName("Chunks");
const FName IAssetManagerEditorModule::PluginName = FName("GameFeaturePlugins");

const FString FAssetManagerEditorRegistrySource::EditorSourceName = TEXT("Editor");
const FString FAssetManagerEditorRegistrySource::CustomSourceName = TEXT("Custom");

TSharedRef<SWidget> IAssetManagerEditorModule::MakePrimaryAssetTypeSelector(FOnGetPrimaryAssetDisplayText OnGetDisplayText, FOnSetPrimaryAssetType OnSetType, bool bAllowClear, bool bAllowAll)
{
	FOnGetPropertyComboBoxStrings GetStrings = FOnGetPropertyComboBoxStrings::CreateStatic(&IAssetManagerEditorModule::GeneratePrimaryAssetTypeComboBoxStrings, bAllowClear, bAllowAll);
	FOnGetPropertyComboBoxValue GetValue = FOnGetPropertyComboBoxValue::CreateLambda([OnGetDisplayText]
	{
		return OnGetDisplayText.Execute().ToString();
	});
	FOnPropertyComboBoxValueSelected SetValue = FOnPropertyComboBoxValueSelected::CreateLambda([OnSetType](const FString& StringValue)
	{
		OnSetType.Execute(FPrimaryAssetType(*StringValue));
	});

	return PropertyCustomizationHelpers::MakePropertyComboBox(nullptr, GetStrings, GetValue, SetValue);
}

TSharedRef<SWidget> IAssetManagerEditorModule::MakePrimaryAssetIdSelector(FPrimaryAssetIdSelectorArgs Args)
{
	FOnGetContent OnCreateMenuContent = FOnGetContent::CreateLambda([
		OnSetId = Args.OnSetId,
		bAllowClear = Args.bAllowClear,
		AllowedTypes = Args.AllowedTypes,
		AllowedClasses = Args.AllowedClasses,
		DisallowedClasses = Args.DisallowedClasses]()
	{
		FOnShouldFilterAsset AssetFilter = FOnShouldFilterAsset::CreateStatic(&IAssetManagerEditorModule::OnShouldFilterPrimaryAsset, AllowedTypes);
		FOnSetObject OnSetObject = FOnSetObject::CreateLambda([OnSetId](const FAssetData& AssetData)
		{
			FSlateApplication::Get().DismissAllMenus();
			UAssetManager& Manager = UAssetManager::Get();

			FPrimaryAssetId AssetId;
			if (AssetData.IsValid())
			{
				AssetId = Manager.GetPrimaryAssetIdForData(AssetData);
				ensure(AssetId.IsValid());
			}

			OnSetId.ExecuteIfBound(AssetId);
		});

		TArray<UFactory*> NewAssetFactories;

		return PropertyCustomizationHelpers::MakeAssetPickerWithMenu(
			FAssetData(),
			bAllowClear,
			AllowedClasses,
			DisallowedClasses,
			NewAssetFactories,
			AssetFilter,
			OnSetObject,
			FSimpleDelegate());
	});

	TAttribute<FText> OnGetObjectText = TAttribute<FText>::Create(Args.OnGetDisplayText);
	TAttribute<FText> OnGetObjectTooltipText = TAttribute<FText>::Create(Args.OnGetTooltipText);

	return SNew(SComboButton)
		.OnGetMenuContent(OnCreateMenuContent)
		.ContentPadding(FMargin(2.0f, 2.0f))
		.ButtonContent()
		[
			SNew(STextBlock)
			.Text(OnGetObjectText)
			.ToolTipText(OnGetObjectTooltipText)
			.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		];
}

TSharedRef<SWidget> IAssetManagerEditorModule::MakePrimaryAssetIdSelector(FOnGetPrimaryAssetDisplayText OnGetDisplayText, FOnSetPrimaryAssetId OnSetId, bool bAllowClear,
	TArray<FPrimaryAssetType> AllowedTypes, TArray<const UClass*> AllowedClasses, TArray<const UClass*> DisallowedClasses)
{
	return MakePrimaryAssetIdSelector(
		FPrimaryAssetIdSelectorArgs {
			.OnGetDisplayText = OnGetDisplayText,
			.OnGetTooltipText = OnGetDisplayText,
			.OnSetId = OnSetId,
			.bAllowClear = bAllowClear,
			.AllowedTypes = AllowedTypes,
			.AllowedClasses = AllowedClasses,
			.DisallowedClasses = DisallowedClasses
		});
}



void IAssetManagerEditorModule::GeneratePrimaryAssetTypeComboBoxStrings(TArray< TSharedPtr<FString> >& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems, bool bAllowClear, bool bAllowAll)
{
	UAssetManager& AssetManager = UAssetManager::Get();

	TArray<FPrimaryAssetTypeInfo> TypeInfos;

	AssetManager.GetPrimaryAssetTypeInfoList(TypeInfos);
	TypeInfos.Sort([](const FPrimaryAssetTypeInfo& LHS, const FPrimaryAssetTypeInfo& RHS) { return LHS.PrimaryAssetType.LexicalLess(RHS.PrimaryAssetType); });

	// Can the field be cleared
	if (bAllowClear)
	{
		// Add None
		OutComboBoxStrings.Add(MakeShared<FString>(FPrimaryAssetType().ToString()));
		OutToolTips.Add(SNew(SToolTip).Text(LOCTEXT("NoType", "None")));
		OutRestrictedItems.Add(false);
	}

	for (FPrimaryAssetTypeInfo& Info : TypeInfos)
	{
		OutComboBoxStrings.Add(MakeShared<FString>(Info.PrimaryAssetType.ToString()));

		FText TooltipText = FText::Format(LOCTEXT("ToolTipFormat", "{0}:{1}{2}"),
			FText::FromString(Info.PrimaryAssetType.ToString()),
			Info.bIsEditorOnly ? LOCTEXT("EditorOnly", " EditorOnly") : FText(),
			Info.bHasBlueprintClasses ? LOCTEXT("Blueprints", " Blueprints") : FText());

		OutToolTips.Add(SNew(SToolTip).Text(TooltipText));
		OutRestrictedItems.Add(false);
	}

	if (bAllowAll)
	{
		// Add All
		OutComboBoxStrings.Add(MakeShared<FString>(IAssetManagerEditorModule::AllPrimaryAssetTypes.ToString()));
		OutToolTips.Add(SNew(SToolTip).Text(LOCTEXT("AllTypes", "All Primary Asset Types")));
		OutRestrictedItems.Add(false);
	}
}

bool IAssetManagerEditorModule::OnShouldFilterPrimaryAsset(const FAssetData& InAssetData, TArray<FPrimaryAssetType> AllowedTypes)
{
	// Make sure it has a primary asset id, and do type check
	UAssetManager& Manager = UAssetManager::Get();

	if (InAssetData.IsValid())
	{
		FPrimaryAssetId AssetId = Manager.GetPrimaryAssetIdForData(InAssetData);
		if (AssetId.IsValid())
		{
			if (AllowedTypes.Num() > 0)
			{
				if (!AllowedTypes.Contains(AssetId.PrimaryAssetType))
				{
					return true;
				}
			}

			return false;
		}
	}

	return true;
}

FAssetData IAssetManagerEditorModule::CreateFakeAssetDataFromChunkId(int32 ChunkID)
{
	return CreateFakeAssetDataFromPrimaryAssetId(UAssetManager::CreatePrimaryAssetIdFromChunkId(ChunkID));
}

int32 IAssetManagerEditorModule::ExtractChunkIdFromFakeAssetData(const FAssetData& InAssetData)
{
	return UAssetManager::ExtractChunkIdFromPrimaryAssetId(ExtractPrimaryAssetIdFromFakeAssetData(InAssetData));
}

FAssetData IAssetManagerEditorModule::CreateFakeAssetDataFromPrimaryAssetId(const FPrimaryAssetId& PrimaryAssetId)
{
	FString PackageNameString = PrimaryAssetFakeAssetDataPackagePath.ToString() / PrimaryAssetId.PrimaryAssetType.ToString();

	// Need to make sure the package part of FTopLevelAssetPath is set otherwise it's gonna be invalid
	FTopLevelAssetPath FakeAssetClass(PrimaryAssetId.PrimaryAssetType, PrimaryAssetId.PrimaryAssetType);
	return FAssetData(*PackageNameString, PrimaryAssetFakeAssetDataPackagePath, PrimaryAssetId.PrimaryAssetName, FakeAssetClass);
}

FPrimaryAssetId IAssetManagerEditorModule::ExtractPrimaryAssetIdFromFakeAssetData(const FAssetData& InAssetData)
{
	if (InAssetData.PackagePath == PrimaryAssetFakeAssetDataPackagePath)
	{
		// See how CreateFakeAssetDataFromPrimaryAssetId stores the asset type inside of FTopLevelAssetPath
		return FPrimaryAssetId(InAssetData.AssetClassPath.GetAssetName(), InAssetData.AssetName);
	}
	return FPrimaryAssetId();
}

void IAssetManagerEditorModule::ExtractAssetIdentifiersFromAssetDataList(const TArray<FAssetData>& AssetDataList, TArray<FAssetIdentifier>& OutAssetIdentifiers)
{
	for (const FAssetData& AssetData : AssetDataList)
	{
		FPrimaryAssetId PrimaryAssetId = IAssetManagerEditorModule::ExtractPrimaryAssetIdFromFakeAssetData(AssetData);

		if (PrimaryAssetId.IsValid())
		{
			OutAssetIdentifiers.Add(PrimaryAssetId);
		}
		else
		{
			OutAssetIdentifiers.Add(AssetData.PackageName);
		}
	}
}

// Concrete implementation

class FAssetManagerEditorModule : public IAssetManagerEditorModule
{
public:
	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End of IModuleInterface interface

	void PerformAuditConsoleCommand(const TArray<FString>& Args);
	void PerformDependencyChainConsoleCommand(const TArray<FString>& Args);
	void PerformDependencyClassConsoleCommand(const TArray<FString>& Args);
	void DumpAssetRegistry(const TArray<FString>& Args);
	void DumpAssetDependencies(const TArray<FString>& Args);

	virtual void OpenAssetAuditUI(TArray<FAssetData> SelectedAssets) override;
	virtual void OpenAssetAuditUI(TArray<FAssetIdentifier> SelectedIdentifiers) override;
	virtual void OpenAssetAuditUI(TArray<FName> SelectedPackages) override;
	virtual FCanOpenReferenceViewerUI& OnCanOpenReferenceViewerUI() override;
	virtual void OpenReferenceViewerUI(const TArray<FAssetIdentifier> SelectedIdentifiers, const FReferenceViewerParams ReferenceViewerParams = FReferenceViewerParams()) override;
	virtual void OpenReferenceViewerUI(const TArray<FName> SelectedPackages, const FReferenceViewerParams ReferenceViewerParams = FReferenceViewerParams()) override;
	virtual bool GetReferenceViewerUISelectedNodesAssetData(TArray<FAssetData>& OutAssetDatas) override;
	virtual void OpenSizeMapUI(TArray<FAssetIdentifier> SelectedIdentifiers) override;
	virtual void OpenSizeMapUI(TArray<FName> SelectedPackages) override;	
	virtual void OpenShaderCookStatistics(TArray<FName> SelectedIdentifiers) override;
	virtual void OpenAssetSizeTreeViewUI(const FString& SelectedPlugin) override;
	virtual bool GetStringValueForCustomColumn(const FAssetData& AssetData, FName ColumnName, FString& OutValue, const FAssetManagerEditorRegistrySource* OverrideRegistrySource = nullptr) override;
	virtual bool GetDisplayTextForCustomColumn(const FAssetData& AssetData, FName ColumnName, FText& OutValue, const FAssetManagerEditorRegistrySource* OverrideRegistrySource = nullptr) override;
	virtual bool GetIntegerValueForCustomColumn(const FAssetData& AssetData, FName ColumnName, int64& OutValue, const FAssetManagerEditorRegistrySource* OverrideRegistrySource = nullptr) override;
	virtual bool GetManagedPackageListForAssetData(const FAssetData& AssetData, TSet<FName>& ManagedPackageSet) override;
	virtual void GetAvailableRegistrySources(TArray<const FAssetManagerEditorRegistrySource*>& AvailableSources) override;
	virtual const FAssetManagerEditorRegistrySource* GetCurrentRegistrySource(bool bNeedManagementData = false) override;
	virtual void SetCurrentRegistrySource(const FString& SourceName) override;
	virtual bool PopulateRegistrySource(FAssetManagerEditorRegistrySource* OutRegistrySource, const FString* OptInFilePath = nullptr) override;
	virtual void RefreshRegistryData() override;
	virtual bool IsPackageInCurrentRegistrySource(FName PackageName) override;
	virtual bool FilterAssetIdentifiersForCurrentRegistrySource(TArray<FAssetIdentifier>& AssetIdentifiers, const FAssetManagerDependencyQuery& DependencyQuery = FAssetManagerDependencyQuery::None(), bool bForwardDependency = true) override;
	virtual bool WriteCollection(FName CollectionName, ECollectionShareType::Type ShareType, const TArray<FName>& PackageNames, bool bShowFeedback) override;
	virtual bool WriteCollection(ICollectionContainer& CollectionContainer, FName CollectionName, ECollectionShareType::Type ShareType, const TArray<FName>& PackageNames, bool bShowFeedback) override;
	
	virtual void AppendRegisteredCommandsToCommandList(TSharedRef<FUICommandList> ListToAppendTo) override;
	virtual FRegisterAdditionalReferenceViewerCommands& OnRegisterAdditionalReferenceViewerCommands() override { return RegisteredCommands; }
	virtual const TArray<TSharedPtr<FUICommandInfo>>& GetRegisteredAdditionalReferenceViewerCommands() override { return AdditionalCommandsInfoList; }
	virtual TSharedRef<SWidget> CreateAssetSourceControlContextMenu(const TArray<FAssetData>& SelectedAssets) override;

private:

	static bool GetDependencyTypeArg(const FString& Arg, UE::AssetRegistry::EDependencyQuery& OutRequiredFlags);

	/**
	 * Tries to open a separate Reference Viewer for the specified Asset Identifiers.
	 * If a Reference Viewer Tab is already showing the same Assets selection, it will be focused and no new Tab will be created
	 */
	void OpenReferenceViewerTab(const TArray<FAssetIdentifier>& InAssetIdentifiers, const FReferenceViewerParams& ReferenceViewerParams);

	//Prints all dependency chains from assets in the search path to the target package.
	void FindReferenceChains(FName TargetPackageName, FName RootSearchPath, UE::AssetRegistry::EDependencyQuery RequiredDependencyFlags);

	//Prints all dependency chains from the PackageName to any dependency of one of the given class names.
	//If the package name is a path rather than a package, then it will do this for each package in the path.
	void FindClassDependencies(FName PackagePath, const TArray<FTopLevelAssetPath>& TargetClasses, UE::AssetRegistry::EDependencyQuery RequiredDependencyFlags);

	void CopyPackageReferences(const TArray<FAssetData> SelectedPackages);
	void CopyPackagePaths(const TArray<FAssetData> SelectedPackages);

	bool GetPackageDependencyChain(FName SourcePackage, FName TargetPackage, TArray<FName>& VisitedPackages, TArray<FName>& OutDependencyChain, UE::AssetRegistry::EDependencyQuery RequiredFlags);
	void GetPackageDependenciesPerClass(FName SourcePackage, const TArray<FTopLevelAssetPath>& TargetClasses, TArray<FName>& VisitedPackages, TArray<FName>& OutDependentPackages, UE::AssetRegistry::EDependencyQuery RequiredDependencyFlags);

	void LogAssetsWithMultipleLabels();
	bool CreateOrEmptyCollection(ICollectionContainer& CollectionContainer, FName CollectionName, ECollectionShareType::Type ShareType, FText& OutError);
	void WriteProfileFile(const FString& Extension, const FString& FileContents);
	
	FString GetSavedAssetRegistryPath(ITargetPlatform* TargetPlatform);
	void GetAssetDataInPaths(const TArray<FString>& Paths, TArray<FAssetData>& OutAssetData);
	bool AreLevelEditorPackagesSelected();
	TArray<FName> GetLevelEditorSelectedAssetPackages();
	TArray<FName> GetContentBrowserSelectedAssetPackages(FOnContentBrowserGetSelection GetSelectionDelegate);
	void InitializeRegistrySources(bool bNeedManagementData);

	TArray<IConsoleObject*> AuditCmds;

	static const TCHAR* FindDepChainHelpText;
	static const TCHAR* FindClassDepHelpText;
	static const FName AssetAuditTabName;
	static const FName AssetDiskSizeTabName;
	static const FName AssetDiskSize2TabName;
	static const FName ReferenceViewerTabName;
	static const FName SizeMapTabName;

	FDelegateHandle ContentBrowserCommandExtenderDelegateHandle;
	FDelegateHandle ReferenceViewerDelegateHandle;
	FDelegateHandle AssetEditorExtenderDelegateHandle;

	/** Currently opened Reference Viewer Tabs, indexed by their Tab ID*/
	TMap<FName, TWeakPtr<SDockTab>> ReferenceViewerTabs;

	/** Associates hash created from assets selection to an existing Reference Viewer Tab ID */
	TMap<uint32, FName> AssetsHashToTabID;

	/** Used to generate unique Tab IDs */
	uint32 GlobalTabCount = 0;

	TWeakPtr<SDockTab> AssetAuditTab;
	TWeakPtr<SDockTab> AssetDiskSizeTab;
	TWeakPtr<SDockTab> SizeMapTab;
	TWeakPtr<SAssetAuditBrowser> AssetAuditUI;
	TWeakPtr<SAssetTableTreeView> AssetDiskSizeUI;
	TWeakPtr<SSizeMap> SizeMapUI;
	TWeakPtr<SReferenceViewer> ReferenceViewerUI;
	TMap<FString, FAssetManagerEditorRegistrySource> RegistrySourceMap;
	FAssetManagerEditorRegistrySource* CurrentRegistrySource;

	IAssetRegistry* AssetRegistry;
	TUniquePtr<FSandboxPlatformFile> CookedSandbox;
	TUniquePtr<FSandboxPlatformFile> EditorCookedSandbox;
	TSharedPtr<FAssetManagerGraphPanelNodeFactory> AssetManagerGraphPanelNodeFactory;
	TSharedPtr<FAssetManagerGraphPanelPinFactory> AssetManagerGraphPanelPinFactory;

	FAssetSourceControlContextMenu AssetSourceControlContextMenu;

	FCanOpenReferenceViewerUI CanOpenReferenceViewerUIDelegate;

	/**Delegate for registering additional Commands */
	FRegisterAdditionalReferenceViewerCommands RegisteredCommands;

	TArray<TSharedPtr<FUICommandInfo>> AdditionalCommandsInfoList;

	/**
	 * When currently selected assets are changed for an already opened Reference Viewer Tab,
	 * this function can be called to properly update Tab Hash and Label
	 */
	void UpdateReferenceViewerTabHashAndLabel(const TArray<FAssetIdentifier>& InPreviousSelection, const TArray<FAssetIdentifier>& InCurrentSelection);

	/** Returns a unique Hash for the specified list of Asset Identifiers, and an optional Label to represent them */
	static uint32 GetHashFromAssetsSelection(const TArray<FAssetIdentifier>& InAssetIdentifiers, FName* OutLabel = nullptr);

	void OnReferenceViewerSelectionChanged(const TArray<FAssetIdentifier>& InPreviousSelection, const TArray<FAssetIdentifier>& InNewSelection);

	void CreateAssetContextMenu(FToolMenuSection& InSection);
	void OnExtendContentBrowserCommands(TSharedRef<FUICommandList> CommandList, FOnContentBrowserGetSelection GetSelectionDelegate);
	void OnExtendLevelEditorCommands(TSharedRef<FUICommandList> CommandList);
	void RegisterMenus();
	void ExtendContentBrowserAssetSelectionMenu();
	void ExtendContentBrowserPathSelectionMenu();
	TSharedRef<FExtender> OnExtendAssetEditor(const TSharedRef<FUICommandList> CommandList, const TArray<UObject*> ContextSensitiveObjects);
	void ExtendAssetEditorMenu();
	void ExtendLevelEditorActorContextMenu();
	void OnReloadComplete(EReloadCompleteReason Reason);
	void OnMarkPackageDirty(UPackage* Pkg, bool bWasDirty);
	void OnEditAssetIdentifiers(TArray<FAssetIdentifier> AssetIdentifiers);
	void OnReferenceViewerTabClosed(TSharedRef<SDockTab> InClosedTab);

	TSharedRef<SDockTab> SpawnAssetAuditTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnAssetDiskSizeTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnReferenceViewerTab(const FSpawnTabArgs& Args);
	TSharedRef<SDockTab> SpawnSizeMapTab(const FSpawnTabArgs& Args);
};

const TCHAR* FAssetManagerEditorModule::FindDepChainHelpText = TEXT("Finds all dependency chains from assets in the given search path, to the target package.\n Usage: FindDepChain TargetPackagePath SearchRootPath (optional: -hardonly/-softonly)\n e.g. FindDepChain /game/characters/heroes/muriel/meshes/muriel /game/cards ");
const TCHAR* FAssetManagerEditorModule::FindClassDepHelpText = TEXT("Finds all dependencies of a certain set of classes to the target asset.\n Usage: FindDepClasses TargetPackagePath ClassName1 ClassName2 etc (optional: -hardonly/-softonly) \n e.g. FindDepChain /game/characters/heroes/muriel/meshes/muriel /game/cards");
const FName FAssetManagerEditorModule::AssetAuditTabName = TEXT("AssetAudit");
const FName FAssetManagerEditorModule::AssetDiskSizeTabName = TEXT("AssetDiskSize");
const FName FAssetManagerEditorModule::AssetDiskSize2TabName = TEXT("AssetDiskSize2");
const FName FAssetManagerEditorModule::ReferenceViewerTabName = TEXT("ReferenceViewer");
const FName FAssetManagerEditorModule::SizeMapTabName = TEXT("SizeMap");

///////////////////////////////////////////

IMPLEMENT_MODULE(FAssetManagerEditorModule, AssetManagerEditor);


void FAssetManagerEditorModule::StartupModule()
{
	CookedSandbox = nullptr;
	EditorCookedSandbox = nullptr;
	CurrentRegistrySource = nullptr;

	// Load the source control module to ensure that it outlives our use of it.
	FModuleManager::Get().LoadModuleChecked<ISourceControlModule>(TEXT("SourceControl"));

	// Load the tree map module so we can use it for size map
	FModuleManager::Get().LoadModule(TEXT("TreeMap"));

	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistry = &AssetRegistryModule.Get();

	FAssetManagerEditorCommands::Register();

	if (GIsEditor && !IsRunningCommandlet())
	{
		AuditCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("AssetManager.AssetAudit"),
			TEXT("Dumps statistics about assets to the log."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAssetManagerEditorModule::PerformAuditConsoleCommand),
			ECVF_Default
			));

		AuditCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("AssetManager.FindDepChain"),
			FindDepChainHelpText,
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAssetManagerEditorModule::PerformDependencyChainConsoleCommand),
			ECVF_Default
			));

		AuditCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("AssetManager.FindDepClasses"),
			FindClassDepHelpText,
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAssetManagerEditorModule::PerformDependencyClassConsoleCommand),
			ECVF_Default
			));

	#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
		AuditCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("AssetManager.DumpAssetRegistry"),
			TEXT("Prints entries in the asset registry. Arguments are required: ObjectPath, PackageName, Path, Class, Tag, Dependencies, PackageData."),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAssetManagerEditorModule::DumpAssetRegistry),
			ECVF_Default
		));
	#endif

		AuditCmds.Add(IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("AssetManager.DumpAssetDependencies"),
			TEXT("Shows a list of all primary assets and the secondary assets that they depend on. Also writes out a .graphviz file"),
			FConsoleCommandWithArgsDelegate::CreateRaw(this, &FAssetManagerEditorModule::DumpAssetDependencies),
			ECVF_Default
			));

		// Register customizations
		FPropertyEditorModule& PropertyModule = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor");
		PropertyModule.RegisterCustomPropertyTypeLayout("PrimaryAssetType", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPrimaryAssetTypeCustomization::MakeInstance));
		PropertyModule.RegisterCustomPropertyTypeLayout("PrimaryAssetId", FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FPrimaryAssetIdCustomization::MakeInstance));
		PropertyModule.NotifyCustomizationModuleChanged();

		// Register Pins and nodes
		AssetManagerGraphPanelPinFactory = MakeShareable(new FAssetManagerGraphPanelPinFactory());
		FEdGraphUtilities::RegisterVisualPinFactory(AssetManagerGraphPanelPinFactory);

		AssetManagerGraphPanelNodeFactory = MakeShareable(new FAssetManagerGraphPanelNodeFactory());
		FEdGraphUtilities::RegisterVisualNodeFactory(AssetManagerGraphPanelNodeFactory);

		// Register content browser hook
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

		TArray<FContentBrowserCommandExtender>& CBCommandExtenderDelegates = ContentBrowserModule.GetAllContentBrowserCommandExtenders();
		CBCommandExtenderDelegates.Add(FContentBrowserCommandExtender::CreateRaw(this, &FAssetManagerEditorModule::OnExtendContentBrowserCommands));
		ContentBrowserCommandExtenderDelegateHandle = CBCommandExtenderDelegates.Last().GetHandle();

		// Register asset editor hooks
		TArray<FAssetEditorExtender>& AssetEditorMenuExtenderDelegates = FAssetEditorToolkit::GetSharedMenuExtensibilityManager()->GetExtenderDelegates();
		AssetEditorMenuExtenderDelegates.Add(FAssetEditorExtender::CreateRaw(this, &FAssetManagerEditorModule::OnExtendAssetEditor));
		AssetEditorExtenderDelegateHandle = AssetEditorMenuExtenderDelegates.Last().GetHandle();
		ExtendAssetEditorMenu();

		// Register level editor hooks and commands
		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");

		TSharedRef<FUICommandList> CommandList = LevelEditorModule.GetGlobalLevelEditorActions();
		OnExtendLevelEditorCommands(CommandList);

		// Add nomad tabs
		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(AssetAuditTabName, FOnSpawnTab::CreateRaw(this, &FAssetManagerEditorModule::SpawnAssetAuditTab))
			.SetDisplayName(LOCTEXT("AssetAuditTitle", "Asset Audit"))
			.SetTooltipText(LOCTEXT("AssetAuditTooltip", "Open Asset Audit window, allows viewing detailed information about assets."))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsAuditCategory())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Audit"));
		FGlobalTabmanager::Get()->RegisterDefaultTabWindowSize(AssetAuditTabName, FVector2D(1080, 600));

		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(AssetDiskSizeTabName, FOnSpawnTab::CreateRaw(this, &FAssetManagerEditorModule::SpawnAssetDiskSizeTab))
			.SetDisplayName(LOCTEXT("AssetDiskSize1Title", "Asset Disk Size 1"))
			.SetTooltipText(LOCTEXT("AssetDiskSize1Tooltip", "Open Asset Disk Size window, allows to analyze disk size for assets."))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsAuditCategory())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Audit"));
		FGlobalTabmanager::Get()->RegisterDefaultTabWindowSize(AssetDiskSizeTabName, FVector2D(1080, 600));

		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(AssetDiskSize2TabName, FOnSpawnTab::CreateRaw(this, &FAssetManagerEditorModule::SpawnAssetDiskSizeTab))
			.SetDisplayName(LOCTEXT("AssetDiskSize2Title", "Asset Disk Size 2"))
			.SetTooltipText(LOCTEXT("AssetDiskSize2Tooltip", "Open Asset Disk Size window, allows to analyze disk size for assets."))
			.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsAuditCategory())
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Audit"));
		FGlobalTabmanager::Get()->RegisterDefaultTabWindowSize(AssetDiskSize2TabName, FVector2D(1080, 600));

		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ReferenceViewerTabName, FOnSpawnTab::CreateRaw(this, &FAssetManagerEditorModule::SpawnReferenceViewerTab))
			.SetDisplayName(FText::Format(LOCTEXT("ReferenceViewerTitle", "{0}"), FText::FromName(UE::AssetManagerEditor::Private::ReferenceViewerLabel)))
			.SetMenuType(ETabSpawnerMenuType::Hidden)
			.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.ReferenceViewer"));

		FGlobalTabmanager::Get()->RegisterNomadTabSpawner(SizeMapTabName, FOnSpawnTab::CreateRaw(this, &FAssetManagerEditorModule::SpawnSizeMapTab))
			.SetDisplayName(LOCTEXT("SizeMapTitle", "Size Map"))
			.SetMenuType(ETabSpawnerMenuType::Hidden);

		// Register for hot reload and package dirty to invalidate data
		FCoreUObjectDelegates::ReloadCompleteDelegate.AddRaw(this, &FAssetManagerEditorModule::OnReloadComplete);

		UPackage::PackageMarkedDirtyEvent.AddRaw(this, &FAssetManagerEditorModule::OnMarkPackageDirty);

		// Register view callbacks
		FEditorDelegates::OnOpenReferenceViewer.AddRaw(this, &FAssetManagerEditorModule::OpenReferenceViewerUI);
		FEditorDelegates::OnOpenSizeMap.AddRaw(this, &FAssetManagerEditorModule::OpenSizeMapUI);
		FEditorDelegates::OnOpenAssetAudit.AddRaw(this, &FAssetManagerEditorModule::OpenAssetAuditUI);
		FEditorDelegates::OnEditAssetIdentifiers.AddRaw(this, &FAssetManagerEditorModule::OnEditAssetIdentifiers);

		// Register callback for extending tool menus
		UToolMenus::RegisterStartupCallback(
			FSimpleMulticastDelegate::FDelegate::CreateRaw(this, &FAssetManagerEditorModule::RegisterMenus));
	}
}

void FAssetManagerEditorModule::ShutdownModule()
{
	CookedSandbox.Reset();
	EditorCookedSandbox.Reset();

	for (IConsoleObject* AuditCmd : AuditCmds)
	{
		IConsoleManager::Get().UnregisterConsoleObject(AuditCmd);
	}
	AuditCmds.Empty();

	if ((GIsEditor && !IsRunningCommandlet()) && UObjectInitialized() && FSlateApplication::IsInitialized())
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));
		
		TArray<FContentBrowserCommandExtender>& CBCommandExtenderDelegates = ContentBrowserModule.GetAllContentBrowserCommandExtenders();
		CBCommandExtenderDelegates.RemoveAll([this](const FContentBrowserCommandExtender& Delegate) { return Delegate.GetHandle() == ContentBrowserCommandExtenderDelegateHandle; });
		
		FGraphEditorModule& GraphEdModule = FModuleManager::LoadModuleChecked<FGraphEditorModule>(TEXT("GraphEditor"));
		
		TArray<FGraphEditorModule::FGraphEditorMenuExtender_SelectedNode>& ReferenceViewerMenuExtenderDelegates = GraphEdModule.GetAllGraphEditorContextMenuExtender();
		ReferenceViewerMenuExtenderDelegates.RemoveAll([this](const FGraphEditorModule::FGraphEditorMenuExtender_SelectedNode& Delegate) { return Delegate.GetHandle() == ReferenceViewerDelegateHandle; });

		TArray<FAssetEditorExtender>& AssetEditorMenuExtenderDelegates = FAssetEditorToolkit::GetSharedMenuExtensibilityManager()->GetExtenderDelegates();
		AssetEditorMenuExtenderDelegates.RemoveAll([this](const FAssetEditorExtender& Delegate) { return Delegate.GetHandle() == AssetEditorExtenderDelegateHandle; });

		FLevelEditorModule& LevelEditorModule = FModuleManager::Get().LoadModuleChecked<FLevelEditorModule>("LevelEditor");

		TSharedRef<FUICommandList> CommandList = LevelEditorModule.GetGlobalLevelEditorActions();
		CommandList->UnmapAction(FAssetManagerEditorCommands::Get().ViewReferences);
		CommandList->UnmapAction(FAssetManagerEditorCommands::Get().ViewSizeMap);
		CommandList->UnmapAction(FAssetManagerEditorCommands::Get().ViewAssetAudit);
		CommandList->UnmapAction(FAssetManagerEditorCommands::Get().ViewShaderCookStatistics);

		if (AssetManagerGraphPanelNodeFactory.IsValid())
		{
			FEdGraphUtilities::UnregisterVisualNodeFactory(AssetManagerGraphPanelNodeFactory);
		}

		if (AssetManagerGraphPanelPinFactory.IsValid())
		{
			FEdGraphUtilities::UnregisterVisualPinFactory(AssetManagerGraphPanelPinFactory);
		}

		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(AssetAuditTabName);
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(AssetDiskSizeTabName);
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(ReferenceViewerTabName);
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(SizeMapTabName);

		if (AssetAuditTab.IsValid())
		{
			AssetAuditTab.Pin()->RequestCloseTab();
		}

		if (!ReferenceViewerTabs.IsEmpty())
		{
			for (const TPair<FName, TWeakPtr<SDockTab>>& Pair : ReferenceViewerTabs)
			{
				if (const TSharedPtr<SDockTab>& Tab = Pair.Value.Pin())
				{
					if (const TSharedPtr<SReferenceViewer>& ReferenceViewer = StaticCastSharedRef<SReferenceViewer>(Tab->GetContent()))
					{
						ReferenceViewer->OnReferenceViewerSelectionChanged().RemoveAll(this);
					}

					Tab->RequestCloseTab();
				}
			}

			ReferenceViewerTabs.Empty();
			AssetsHashToTabID.Empty();
		}

		if (SizeMapTab.IsValid())
		{
			SizeMapTab.Pin()->RequestCloseTab();
		}

		FCoreUObjectDelegates::ReloadCompleteDelegate.RemoveAll(this);

		UPackage::PackageMarkedDirtyEvent.RemoveAll(this);
		FEditorDelegates::OnOpenReferenceViewer.RemoveAll(this);
		FEditorDelegates::OnOpenSizeMap.RemoveAll(this);
		FEditorDelegates::OnOpenAssetAudit.RemoveAll(this);
		FEditorDelegates::OnEditAssetIdentifiers.RemoveAll(this);

		// Cleanup tool menus
		UToolMenus::UnRegisterStartupCallback(this);
		UToolMenus::UnregisterOwner(this);
	}
}

TSharedRef<SDockTab> FAssetManagerEditorModule::SpawnAssetAuditTab(const FSpawnTabArgs& Args)
{
	check(UAssetManager::IsInitialized());

	TSharedRef<SDockTab> DockTab = SAssignNew(AssetAuditTab, SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SAssignNew(AssetAuditUI, SAssetAuditBrowser)
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda([this](TSharedRef<SDockTab>)
		{
			if (AssetAuditUI.IsValid())
			{
				AssetAuditUI.Pin()->OnClose();
			}
		}));

	return DockTab;
}

TSharedRef<SDockTab> FAssetManagerEditorModule::SpawnAssetDiskSizeTab(const FSpawnTabArgs& Args)
{
	check(UAssetManager::IsInitialized());

	TSharedRef<FAssetTable> AssetTable = MakeShared<FAssetTable>();
	AssetTable->Reset();
	AssetTable->SetDisplayName(FText::FromString(TEXT("AssetTable")));

	TSharedRef<SDockTab> DockTab = SAssignNew(AssetDiskSizeTab, SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SAssignNew(AssetDiskSizeUI, SAssetTableTreeView, AssetTable)
		];

	DockTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateLambda([this](TSharedRef<SDockTab>)
		{
			if (AssetDiskSizeUI.IsValid())
			{
				AssetDiskSizeUI.Pin()->OnClose();
			}
		}));

	return DockTab;
}

TSharedRef<SDockTab> FAssetManagerEditorModule::SpawnReferenceViewerTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> NewTab = SNew(SDockTab)
		.TabRole(NomadTab)
		.OnTabClosed_Raw(this, &FAssetManagerEditorModule::OnReferenceViewerTabClosed);

	TSharedRef<SReferenceViewer> ReferenceViewerWidget = SNew(SReferenceViewer).ParentTab(NewTab);
	ReferenceViewerUI = ReferenceViewerWidget;
	NewTab->SetContent(ReferenceViewerWidget);

	if (!ReferenceViewerTabs.Contains(ReferenceViewerTabName))
	{
		// This is probably the Reference Viewer Tab created when restoring Editor Layout.
		ReferenceViewerTabs.Emplace(ReferenceViewerTabName, NewTab);
	}

	return NewTab;
}

TSharedRef<SDockTab> FAssetManagerEditorModule::SpawnSizeMapTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> NewTab = SAssignNew(SizeMapTab, SDockTab)
		.TabRole(ETabRole::NomadTab);

	NewTab->SetContent(SAssignNew(SizeMapUI, SSizeMap));

	return NewTab;
}

void FAssetManagerEditorModule::OpenAssetAuditUI(TArray<FAssetData> SelectedAssets)
{
	FGlobalTabmanager::Get()->TryInvokeTab(AssetAuditTabName);

	if (AssetAuditUI.IsValid())
	{
		AssetAuditUI.Pin()->AddAssetsToList(SelectedAssets, false);
	}
}

void FAssetManagerEditorModule::OpenAssetAuditUI(TArray<FAssetIdentifier> SelectedIdentifiers)
{
	FGlobalTabmanager::Get()->TryInvokeTab(AssetAuditTabName);

	if (AssetAuditUI.IsValid())
	{
		AssetAuditUI.Pin()->AddAssetsToList(SelectedIdentifiers, false);
	}
}

void FAssetManagerEditorModule::OpenAssetAuditUI(TArray<FName> SelectedPackages)
{
	FGlobalTabmanager::Get()->TryInvokeTab(AssetAuditTabName);

	if (AssetAuditUI.IsValid())
	{
		AssetAuditUI.Pin()->AddAssetsToList(SelectedPackages, false);
	}
}

IAssetManagerEditorModule::FCanOpenReferenceViewerUI& FAssetManagerEditorModule::OnCanOpenReferenceViewerUI()
{
	return CanOpenReferenceViewerUIDelegate;
}

void FAssetManagerEditorModule::OpenReferenceViewerTab(const TArray<FAssetIdentifier>& InAssetIdentifiers, const FReferenceViewerParams& ReferenceViewerParams)
{
	FName TabLabel;
	uint32 SelectionHash = GetHashFromAssetsSelection(InAssetIdentifiers, &TabLabel);

	FName TabID;

	// Look for possibly existing Tab for the same assets
	if (const FName* TabIDPtr = AssetsHashToTabID.Find(SelectionHash))
	{
		TabID = *TabIDPtr;

		// Check if there's already a Reference Viewer Tab for this Assets selection
		if (const TWeakPtr<SDockTab>* ExistingTabWeak = ReferenceViewerTabs.Find(TabID))
		{
			if (const TSharedPtr<SDockTab>& ExistingTab = ExistingTabWeak->Pin())
			{
				FGlobalTabmanager::Get()->DrawAttention(ExistingTab.ToSharedRef());
				return;
			}
		}
	}

	TSharedPtr<SDockTab> ReferenceViewerDefaultTab;

	// Look for the default reference viewer tab, in case an empty one already exists (e.g. from a previous Editor session)
	if (const TWeakPtr<SDockTab>* ReferenceViewerDefaultTabWeak = ReferenceViewerTabs.Find(ReferenceViewerTabName))
	{
		ReferenceViewerDefaultTab = ReferenceViewerDefaultTabWeak->Pin();
	}

	// Look through Tab Manager as well
	if (!ReferenceViewerDefaultTab.IsValid())
	{
		ReferenceViewerDefaultTab = FGlobalTabmanager::Get()->FindExistingLiveTab(ReferenceViewerTabName);
	}

	// FindExistingLiveTab fails when looking for tabs located e.g. in details panel area, etc.
	// So, we make sure we are actually getting a reference to an empty Reference Viewer Tab.
	// This means its label is still not set to an asset name
	if (!ReferenceViewerDefaultTab.IsValid())
	{
		if (!AssetsHashToTabID.IsEmpty())
		{
			// Let's get the last opened tab, so we use it as reference for where new tabs should go
			TArray<FName> TabNames;
			AssetsHashToTabID.GenerateValueArray(TabNames);
			TabNames.Sort(
				[](const FName A, const FName B)
				{
					return A.ToString() < B.ToString();
				}
			);
			ReferenceViewerDefaultTab = ReferenceViewerTabs.FindRef(TabNames.Last()).Pin();
		}
		else
		{
			// There's no Reference Viewer Tab, let's retrieve the default one
			ReferenceViewerDefaultTab = FGlobalTabmanager::Get()->TryInvokeTab(ReferenceViewerTabName);
		}
	}

	TSharedPtr<SDockTab> CurrentAssetTab;
	if (ReferenceViewerDefaultTab && AssetsHashToTabID.IsEmpty())
	{
		TabID = ReferenceViewerTabName;
		CurrentAssetTab = ReferenceViewerDefaultTab;
		FGlobalTabmanager::Get()->DrawAttention(ReferenceViewerDefaultTab.ToSharedRef());
	}

	if (!CurrentAssetTab)
	{
		// There is no existing tab yet for this Asset, we need to create a new one.
		CurrentAssetTab = SNew(SDockTab)
			.TabRole(ETabRole::NomadTab);

		// Generate Tab ID 
		TabID = FName(ReferenceViewerTabName.ToString() + "_" + FString::FromInt(++GlobalTabCount));

		// Get reference Tab name, used to insert the new tab at its location
		const FName PlacementRefTabName = ReferenceViewerDefaultTab
											? ReferenceViewerDefaultTab->GetLayoutIdentifier().TabType
											: ReferenceViewerTabName;

		// Try to place the newly created tab next to other Reference Viewer Tabs
		FGlobalTabmanager::Get()->InsertNewDocumentTab(
			PlacementRefTabName,
			TabID,
			FTabManager::FLastMajorOrNomadTab(PlacementRefTabName),
			CurrentAssetTab.ToSharedRef());
	}

	if (CurrentAssetTab)
	{
		// Initialize Tab
		TSharedRef<SReferenceViewer> ReferenceViewerWidget = SNew(SReferenceViewer).ParentTab(CurrentAssetTab.ToSharedRef());
		ReferenceViewerUI = ReferenceViewerWidget;
		CurrentAssetTab->SetContent(ReferenceViewerWidget);
		CurrentAssetTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FAssetManagerEditorModule::OnReferenceViewerTabClosed));
		CurrentAssetTab->SetTabIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.ReferenceViewer").GetIcon());
		CurrentAssetTab->SetLabel(FText::FromName(TabLabel));

		ReferenceViewerTabs.Emplace(TabID, CurrentAssetTab);
		AssetsHashToTabID.Emplace(SelectionHash, TabID);

		const TSharedPtr<SReferenceViewer>& ReferenceViewer = StaticCastSharedRef<SReferenceViewer>(CurrentAssetTab->GetContent());
		ReferenceViewer->SetGraphRootIdentifiers(InAssetIdentifiers, ReferenceViewerParams);

		ReferenceViewer->OnReferenceViewerSelectionChanged().AddRaw(this, &FAssetManagerEditorModule::OnReferenceViewerSelectionChanged);
	}
}

void FAssetManagerEditorModule::OpenReferenceViewerUI(const TArray<FAssetIdentifier> SelectedIdentifiers, const FReferenceViewerParams ReferenceViewerParams)
{
	// True : will create a dedicated Tab for each specified Asset Identifier. Prevents multiple tabs for the same Asset.
	// False: legacy behavior, in which a single tab is available, and re-used.
	bool bOpenInNewTab = true;
	UE::AssetManagerEditor::Private::CVarUseMultipleReferenceViewerTabs->GetValue(bOpenInNewTab);

	if (!SelectedIdentifiers.IsEmpty())
	{
		FText ErrorMessage;
		if (OnCanOpenReferenceViewerUI().IsBound() && !OnCanOpenReferenceViewerUI().Execute(SelectedIdentifiers, &ErrorMessage))
		{
			if (!ErrorMessage.IsEmpty())
			{
				FNotificationInfo Info(ErrorMessage);
				Info.ExpireDuration = 5.0f;
				if (TSharedPtr<SNotificationItem> InfoItem = FSlateNotificationManager::Get().AddNotification(Info))
				{
					InfoItem->SetCompletionState(SNotificationItem::CS_Fail);
				}
			}
		}
		else if (!bOpenInNewTab)
		{
			if (const TSharedPtr<SDockTab>& InvokedTab = FGlobalTabmanager::Get()->TryInvokeTab(ReferenceViewerTabName))
			{
				TSharedPtr<SReferenceViewer> ReferenceViewer = StaticCastSharedRef<SReferenceViewer>(InvokedTab->GetContent());
				ReferenceViewer->SetGraphRootIdentifiers(SelectedIdentifiers, ReferenceViewerParams);
				InvokedTab->SetLabel(FText::FromName(UE::AssetManagerEditor::Private::ReferenceViewerLabel));
			}
		}
		else
		{
			OpenReferenceViewerTab(SelectedIdentifiers, ReferenceViewerParams);
		}
	}
}

void FAssetManagerEditorModule::OpenReferenceViewerUI(const TArray<FName> SelectedPackages, const FReferenceViewerParams ReferenceViewerParams)
{
	TArray<FAssetIdentifier> Identifiers;
	for (FName Name : SelectedPackages)
	{
		Identifiers.Add(FAssetIdentifier(Name));
	}

	OpenReferenceViewerUI(Identifiers, ReferenceViewerParams);
}

bool FAssetManagerEditorModule::GetReferenceViewerUISelectedNodesAssetData(TArray<FAssetData>& OutAssetDatas)
{
	bool ValidAssetDataFound = false;
	for (auto TabPair : ReferenceViewerTabs)
	{
		if (TabPair.Value.IsValid())
		{
			const TSharedPtr<SDockTab> Tab = TabPair.Value.Pin();

			// we only want to do this on the actual active tab
			if (Tab->IsForeground())
			{
				TSharedPtr<SWidget> TabContent = Tab->GetContent();

				if (TabContent->GetWidgetClass().GetWidgetType() != SReferenceViewer::StaticWidgetClass().GetWidgetType())
				{
					continue;
				}

				TSharedPtr<SReferenceViewer> ActiveReferenceViewer = StaticCastSharedPtr<SReferenceViewer>(TabContent);

				TArray<FAssetData> SelectedAssets;
				ActiveReferenceViewer->GetSelectedNodeAssetData(SelectedAssets);

				// just in case we can have two valid tabs displayed at once, writing this in such a way that it should still work
				OutAssetDatas.Append(SelectedAssets);
				ValidAssetDataFound = true;
			}
		}
	}

	return ValidAssetDataFound;
}

void FAssetManagerEditorModule::OpenSizeMapUI(TArray<FName> SelectedPackages)
{
	TArray<FAssetIdentifier> Identifiers;
	for (FName Name : SelectedPackages)
	{
		Identifiers.Add(FAssetIdentifier(Name));
	}

	OpenSizeMapUI(Identifiers);
}

void FAssetManagerEditorModule::OpenSizeMapUI(TArray<FAssetIdentifier> SelectedIdentifiers)
{
	if (SelectedIdentifiers.Num() > 0)
	{
		if (TSharedPtr<SDockTab> NewTab = FGlobalTabmanager::Get()->TryInvokeTab(SizeMapTabName))
		{
			TSharedRef<SSizeMap> SizeMap = StaticCastSharedRef<SSizeMap>(NewTab->GetContent());
			SizeMap->SetRootAssetIdentifiers(SelectedIdentifiers);
		}
	}
}

void FAssetManagerEditorModule::OpenShaderCookStatistics(TArray<FName> SelectedPackages)
{
	FString SubPath;
	FString CommonPath = "";
	if(SelectedPackages.Num())
	{
		//Find the common path
		CommonPath = SelectedPackages[0].ToString();
		uint32 CommonIdentical = CommonPath.Len();
		for (FName Name : SelectedPackages)
		{
			FString Path = Name.ToString();
			uint32 Identical = 0;
			uint32 NumCharacters = FMath::Min((uint32)Path.Len(), CommonIdentical);
			for(Identical = 0; Identical < NumCharacters; ++Identical)
			{
				if(CommonPath[Identical] != Path[Identical])
				{
					break;
				}
			}
			CommonIdentical = FMath::Min(Identical, CommonIdentical);
		}
		CommonPath.LeftInline(CommonIdentical);
	}
	static const FName LevelEditorModuleName("LevelEditor");
	static const FName LevelEditorStatsViewerTab("LevelEditorStatsViewer");
	FLevelEditorModule& LevelEditorModule = FModuleManager::GetModuleChecked<FLevelEditorModule>(LevelEditorModuleName);
	TSharedPtr<FTabManager> TabManager = LevelEditorModule.GetLevelEditorTabManager();
	if (TSharedPtr<SDockTab> Tab = TabManager->TryInvokeTab(LevelEditorStatsViewerTab))
	{
		TSharedRef<SWidget> Content = Tab->GetContent();
		IStatsViewer* StatsView = (IStatsViewer*)&*Content;
		StatsView->SwitchAndFilterPage(EStatsPage::ShaderCookerStats, CommonPath, FString("Path"));
	}
}

void FAssetManagerEditorModule::OpenAssetSizeTreeViewUI(const FString& SearchTerm)
{
	if (TSharedPtr<SDockTab> NewTab = FGlobalTabmanager::Get()->TryInvokeTab(AssetDiskSizeTabName))
	{
		TSharedRef<SAssetTableTreeView> AssetTableTreeView = StaticCastSharedRef<SAssetTableTreeView>(NewTab->GetContent());

		AssetTableTreeView->SetFilterToName(SearchTerm);
	}
}

void FAssetManagerEditorModule::GetAssetDataInPaths(const TArray<FString>& Paths, TArray<FAssetData>& OutAssetData)
{
	// Form a filter from the paths
	FARFilter Filter;
	Filter.bRecursivePaths = true;
	for (const FString& Path : Paths)
	{
		new (Filter.PackagePaths) FName(*Path);
	}

	// Query for a list of assets in the selected paths
	AssetRegistry->GetAssets(Filter, OutAssetData);
}

bool FAssetManagerEditorModule::AreLevelEditorPackagesSelected()
{
	return GetLevelEditorSelectedAssetPackages().Num() > 0;
}

TArray<FName> FAssetManagerEditorModule::GetLevelEditorSelectedAssetPackages()
{
	TArray<FName> OutAssetPackages;
	TArray<UObject*> ReferencedAssets;
	GEditor->GetReferencedAssetsForEditorSelection(ReferencedAssets);

	for (UObject* EditedAsset : ReferencedAssets)
	{
		if (IsValid(EditedAsset) && EditedAsset->IsAsset())
		{
			OutAssetPackages.AddUnique(EditedAsset->GetOutermost()->GetFName());
		}
	}
	return OutAssetPackages;
}

TArray<FName> FAssetManagerEditorModule::GetContentBrowserSelectedAssetPackages(FOnContentBrowserGetSelection GetSelectionDelegate)
{
	TArray<FAssetData> SelectedAssets;
	TArray<FString> SelectedPaths;

	if (GetSelectionDelegate.IsBound())
	{
		TArray<FString> SelectedVirtualPaths;
		GetSelectionDelegate.Execute(SelectedAssets, SelectedVirtualPaths);

		for (const FString& VirtualPath : SelectedVirtualPaths)
		{
			FString InvariantPath;
			if (IContentBrowserDataModule::Get().GetSubsystem()->TryConvertVirtualPath(VirtualPath, InvariantPath) == EContentBrowserPathType::Internal)
			{
				SelectedPaths.Add(InvariantPath);
			}
		}
	}

	GetAssetDataInPaths(SelectedPaths, SelectedAssets);

	TSet<FName> PackageNames;
	PackageNames.Reserve(SelectedAssets.Num());
	for (const FAssetData& AssetData : SelectedAssets)
	{
		PackageNames.Add(AssetData.PackageName);
	}

	return PackageNames.Array();
}

void FAssetManagerEditorModule::UpdateReferenceViewerTabHashAndLabel(const TArray<FAssetIdentifier>& InPreviousSelection, const TArray<FAssetIdentifier>& InCurrentSelection)
{
	uint32 HashToChange = GetHashFromAssetsSelection(InPreviousSelection);

	FName TabLabel;
	uint32 NewHash = GetHashFromAssetsSelection(InCurrentSelection, &TabLabel);

	if (!AssetsHashToTabID.Contains(HashToChange))
	{
		return;
	}

	FName TabId;
	AssetsHashToTabID.RemoveAndCopyValue(HashToChange, TabId);
	AssetsHashToTabID.Emplace(NewHash, TabId);

	if (const TWeakPtr<SDockTab>* TabWeak = ReferenceViewerTabs.Find(TabId))
	{
		if (const TSharedPtr<SDockTab> Tab = TabWeak->Pin())
		{
			Tab->SetLabel(FText::FromName(TabLabel));
		}
	}

	return;
}

uint32 FAssetManagerEditorModule::GetHashFromAssetsSelection(const TArray<FAssetIdentifier>& InAssetIdentifiers, FName* OutLabel)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FAssetManagerEditorModule::GetHashFromAssetsSelection);

	uint32 SelectionHash = 0;
	FString TabLabel;

	if (InAssetIdentifiers.Num() > 0)
	{
		TArray<FName> PackageNames;
		for (const FAssetIdentifier& AssetIdentifier : InAssetIdentifiers)
		{
			PackageNames.Add(AssetIdentifier.PackageName);
		}

		TMap<FName, FAssetData> PackageToAssetDataMap;
		UE::AssetRegistry::GetAssetForPackages(PackageNames, PackageToAssetDataMap);

		if (!PackageToAssetDataMap.IsEmpty())
		{
			TabLabel = PackageToAssetDataMap[PackageNames[0]].AssetName.ToString();
		}

		bool bNoAsset = false;
		// C++ classes will lead to no asset, so we retrieve their package instead
		// (TODO: this needs to be addressed in order to properly show reference viewer graph for C++ Assets)

		if (!PackageToAssetDataMap.Contains(PackageNames[0]))
		{
			bNoAsset = true;
			if (TabLabel.IsEmpty())
			{
				TabLabel = PackageNames[0].ToString();

				if (!InAssetIdentifiers[0].ValueName.IsNone())
				{
					TabLabel += TEXT(":") + InAssetIdentifiers[0].ValueName.ToString();
				}
			}
		}

		// Label for multiple assets matches Path field at the top of Reference Viewer graph
		if (InAssetIdentifiers.Num() > 1)
		{
			TabLabel += TEXT(" and ") + FString::FromInt(InAssetIdentifiers.Num() - 1) + TEXT(" others");
		}

		// Create a hash from the concatenation of package names from all the selected assets. This hash is used to match a selection with a Tab ID
		// This allows to ignore selection order when comparing selections while looking for an existing Reference Viewer for the current selection
		// We sort package names, so that selection order is not be taken into account

		PackageNames.Sort([](const FName& NameA, const FName& NameB)
		{ 
			return NameA.FastLess(NameB);
		});

		for (const FName PackageName : PackageNames)
		{
			// In case there is no asset, we might get the same Hash for potentially different graphs (e.g. GameplayTags),
			// so let's use the tab label, which in that specific case should include both package name and value
			if (bNoAsset)
			{
				SelectionHash ^= GetTypeHash(TabLabel);
			}
			else
			{
				SelectionHash ^= GetTypeHash(PackageName);
			}
		}
	}

	if (OutLabel)
	{
		*OutLabel = FName(TabLabel);
	}

	return SelectionHash;
}

void FAssetManagerEditorModule::OnReferenceViewerSelectionChanged(const TArray<FAssetIdentifier>& InPreviousSelection, const TArray<FAssetIdentifier>& InNewSelection)
{
	UpdateReferenceViewerTabHashAndLabel(InPreviousSelection, InNewSelection);
}

void FAssetManagerEditorModule::CreateAssetContextMenu(FToolMenuSection& InSection)
{
	InSection.AddMenuEntry(
		FAssetManagerEditorCommands::Get().ViewReferences,
		TAttribute<FText>(), // Use command Label
		TAttribute<FText>(), // Use command tooltip
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.ReferenceViewer")
	);
	InSection.AddMenuEntry(
		FAssetManagerEditorCommands::Get().ViewSizeMap,
		TAttribute<FText>(), // Use command Label
		TAttribute<FText>(), // Use command tooltip
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "ContentBrowser.SizeMap")
	);
	InSection.AddMenuEntry(
		FAssetManagerEditorCommands::Get().ViewAssetAudit,
		TAttribute<FText>(), // Use command Label
		TAttribute<FText>(), // Use command tooltip
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Audit")
	);
	InSection.AddMenuEntry(
		FAssetManagerEditorCommands::Get().ViewShaderCookStatistics,
		TAttribute<FText>(), // Use command Label
		TAttribute<FText>(), // Use command tooltip
		FSlateIcon(FAppStyle::GetAppStyleSetName(), "MainFrame.CookContent")
	);

	AssetSourceControlContextMenu.MakeContextMenu(InSection);
}

void FAssetManagerEditorModule::OnExtendContentBrowserCommands(TSharedRef<FUICommandList> CommandList, FOnContentBrowserGetSelection GetSelectionDelegate)
{
	// There is no can execute because the focus state is weird during calls to it
	CommandList->MapAction(FAssetManagerEditorCommands::Get().ViewReferences, 
		FExecuteAction::CreateLambda([this, GetSelectionDelegate]
		{
			OpenReferenceViewerUI(GetContentBrowserSelectedAssetPackages(GetSelectionDelegate));
		})
	);

	CommandList->MapAction(FAssetManagerEditorCommands::Get().ViewSizeMap,
		FExecuteAction::CreateLambda([this, GetSelectionDelegate]
		{
			OpenSizeMapUI(GetContentBrowserSelectedAssetPackages(GetSelectionDelegate));
		})
	);


	CommandList->MapAction(FAssetManagerEditorCommands::Get().ViewShaderCookStatistics,
		FExecuteAction::CreateLambda([this, GetSelectionDelegate]
	{
		OpenShaderCookStatistics(GetContentBrowserSelectedAssetPackages(GetSelectionDelegate));
	})
	);

	CommandList->MapAction(FAssetManagerEditorCommands::Get().ViewAssetAudit,
		FExecuteAction::CreateLambda([this, GetSelectionDelegate]
		{
			OpenAssetAuditUI(GetContentBrowserSelectedAssetPackages(GetSelectionDelegate));
		})
	);
}

void FAssetManagerEditorModule::OnExtendLevelEditorCommands(TSharedRef<FUICommandList> CommandList)
{
	CommandList->MapAction(FAssetManagerEditorCommands::Get().ViewReferences, 
		FExecuteAction::CreateLambda([this]
		{
			OpenReferenceViewerUI(GetLevelEditorSelectedAssetPackages());
		}),
		FCanExecuteAction::CreateRaw(this, &FAssetManagerEditorModule::AreLevelEditorPackagesSelected)
	);

	CommandList->MapAction(FAssetManagerEditorCommands::Get().ViewSizeMap,
		FExecuteAction::CreateLambda([this]
		{
			OpenSizeMapUI(GetLevelEditorSelectedAssetPackages());
		}),
		FCanExecuteAction::CreateRaw(this, &FAssetManagerEditorModule::AreLevelEditorPackagesSelected)
	);

	CommandList->MapAction(FAssetManagerEditorCommands::Get().ViewShaderCookStatistics,
		FExecuteAction::CreateLambda([this]
		{
			OpenShaderCookStatistics(GetLevelEditorSelectedAssetPackages());
		}),
		FCanExecuteAction::CreateRaw(this, &FAssetManagerEditorModule::AreLevelEditorPackagesSelected)
	);



	CommandList->MapAction(FAssetManagerEditorCommands::Get().ViewAssetAudit,
		FExecuteAction::CreateLambda([this]
		{
			OpenAssetAuditUI(GetLevelEditorSelectedAssetPackages());
		}),
		FCanExecuteAction::CreateRaw(this, &FAssetManagerEditorModule::AreLevelEditorPackagesSelected)
	);
}

void FAssetManagerEditorModule::RegisterMenus()
{
	FToolMenuOwnerScoped OwnerScoped(this);

	ExtendContentBrowserAssetSelectionMenu();
	ExtendContentBrowserPathSelectionMenu();
	ExtendLevelEditorActorContextMenu();
}

void FAssetManagerEditorModule::ExtendContentBrowserAssetSelectionMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.AssetContextMenu");
	FToolMenuSection& Section = Menu->FindOrAddSection("AssetContextReferences");
	FToolMenuEntry& Entry = Section.AddDynamicEntry("AssetManagerEditorViewCommands", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
	{
		UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>();
		if (Context && Context->SelectedAssets.Num() > 0)
		{
			CreateAssetContextMenu(InSection);
		}
	}));
}

void FAssetManagerEditorModule::ExtendContentBrowserPathSelectionMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("ContentBrowser.FolderContextMenu");
	FToolMenuSection& Section = Menu->FindOrAddSection("PathContextBulkOperations");
	FToolMenuEntry& Entry = Section.AddDynamicEntry("AssetManagerEditorViewCommands", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
	{
		UContentBrowserFolderContext* Context = InSection.FindContext<UContentBrowserFolderContext>();
		if (Context && Context->NumAssetPaths > 0)
		{
			CreateAssetContextMenu(InSection);
		}
	}));
}

TSharedRef<FExtender> FAssetManagerEditorModule::OnExtendAssetEditor(const TSharedRef<FUICommandList> CommandList, const TArray<UObject*> ContextSensitiveObjects)
{
	TArray<FName> PackageNames;
	TArray<FAssetData> PackageAssets;
	for (UObject* EditedAsset : ContextSensitiveObjects)
	{
		if (IsValid(EditedAsset) && EditedAsset->IsAsset())
		{
			PackageNames.AddUnique(EditedAsset->GetOutermost()->GetFName());
			PackageAssets.AddUnique(FAssetData(EditedAsset));
		}
	}

	TSharedRef<FExtender> Extender(new FExtender());

	if (PackageNames.Num() > 0)
	{
		// It's safe to modify the CommandList here because this is run as the editor UI is created and the payloads are safe
		CommandList->MapAction(
			FAssetManagerEditorCommands::Get().CopyAssetReferences,
			FExecuteAction::CreateRaw(this, &FAssetManagerEditorModule::CopyPackageReferences, PackageAssets));

		CommandList->MapAction(
			FAssetManagerEditorCommands::Get().CopyAssetPaths,
			FExecuteAction::CreateRaw(this, &FAssetManagerEditorModule::CopyPackagePaths, PackageAssets));

		CommandList->MapAction(
			FAssetManagerEditorCommands::Get().ViewReferences,
			FExecuteAction::CreateRaw(this, &FAssetManagerEditorModule::OpenReferenceViewerUI, PackageNames, FReferenceViewerParams()));

		CommandList->MapAction(
			FAssetManagerEditorCommands::Get().ViewSizeMap,
			FExecuteAction::CreateRaw(this, &FAssetManagerEditorModule::OpenSizeMapUI, PackageNames));

		CommandList->MapAction(
			FAssetManagerEditorCommands::Get().ViewShaderCookStatistics,
			FExecuteAction::CreateRaw(this, &FAssetManagerEditorModule::OpenShaderCookStatistics, PackageNames));

		CommandList->MapAction(
			FAssetManagerEditorCommands::Get().ViewAssetAudit,
			FExecuteAction::CreateRaw(this, &FAssetManagerEditorModule::OpenAssetAuditUI, PackageNames));
	}

	return Extender;
}

void FAssetManagerEditorModule::ExtendAssetEditorMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("MainFrame.MainMenu.Asset");
	FToolMenuSection& Section = Menu->FindOrAddSection("AssetEditorActions");
	FToolMenuEntry& Entry = Section.AddDynamicEntry("AssetManagerEditorViewCommands", FNewToolMenuSectionDelegate::CreateLambda([this](FToolMenuSection& InSection)
	{
		UAssetEditorToolkitMenuContext* MenuContext = InSection.FindContext<UAssetEditorToolkitMenuContext>();
		if (MenuContext && MenuContext->Toolkit.IsValid() && MenuContext->Toolkit.Pin()->IsActuallyAnAsset())
		{
			for (const UObject* EditedAsset : *MenuContext->Toolkit.Pin()->GetObjectsCurrentlyBeingEdited())
			{
				if (IsValid(EditedAsset) && EditedAsset->IsAsset())
				{
					InSection.AddMenuEntry(
						FAssetManagerEditorCommands::Get().CopyAssetReferences,
						TAttribute<FText>(), // Use command Label
						TAttribute<FText>(), // Use command tooltip
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy")
					);
					InSection.AddMenuEntry(
						FAssetManagerEditorCommands::Get().CopyAssetPaths,
						TAttribute<FText>(), // Use command Label
						TAttribute<FText>(), // Use command tooltip
						FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy")
					);

					CreateAssetContextMenu(InSection);
					break;
				}
			}
		}
	}));
	Entry.InsertPosition = FToolMenuInsert("FindInContentBrowser", EToolMenuInsertType::After);
}

void FAssetManagerEditorModule::ExtendLevelEditorActorContextMenu()
{
	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.ActorContextMenu.AssetToolsSubMenu");

	FToolMenuSection& Section = Menu->AddSection("AssetManagerEditorViewCommands", TAttribute<FText>(), FToolMenuInsert(NAME_None, EToolMenuInsertType::First));
	CreateAssetContextMenu(Section);
}

void FAssetManagerEditorModule::OnReloadComplete(EReloadCompleteReason Reason)
{
	// Invalidate on a hot reload
	check(UAssetManager::IsInitialized());
	UAssetManager::Get().InvalidatePrimaryAssetDirectory();
}

void FAssetManagerEditorModule::OnMarkPackageDirty(UPackage* Pkg, bool bWasDirty)
{
	if (!UAssetManager::IsInitialized())
	{
		// Can be called before InitializeObjectReferences; do nothing in that case
		return;
	}
	UAssetManager& AssetManager = UAssetManager::Get();

	// Check if this package is managed, if so invalidate
	FPrimaryAssetId AssetId = AssetManager.GetPrimaryAssetIdForPackage(Pkg->GetFName());

	if (AssetId.IsValid())
	{
		AssetManager.InvalidatePrimaryAssetDirectory();
	}
}

void FAssetManagerEditorModule::OnEditAssetIdentifiers(TArray<FAssetIdentifier> AssetIdentifiers)
{
	UAssetManager& AssetManager = UAssetManager::Get();

	// Determine which packages to load
	TArray<FAssetData> AssetsToLoad;
	for (FAssetIdentifier AssetIdentifier : AssetIdentifiers)
	{
		TArray<FAssetData> AssetDataArray;
		if (AssetIdentifier.IsPackage())
		{
			// Directly a package to load
			AssetRegistry->GetAssetsByPackageName(AssetIdentifier.PackageName, AssetDataArray);
		}
		else
		{
			// If it's a primary asset ID, resolve it to a package to load
			FPrimaryAssetId AssetId = AssetIdentifier.GetPrimaryAssetId();
			if (AssetId.IsValid())
			{
				//@TODO: We probably want to call UAssetManager::GetNameData with bCheckRedirector=false but that's protected
				FSoftObjectPath AssetPath = AssetManager.GetPrimaryAssetPath(AssetId);
				FAssetData AssetData;
				if (AssetManager.GetAssetDataForPath(AssetPath, /*out*/ AssetData))
				{
					AssetDataArray.Add(AssetData);
				}
			}
		}

		FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>("AssetTools");

		for (const FAssetData& AssetData : AssetDataArray)
		{
			TWeakPtr<IAssetTypeActions> AssetTypeActionsWeak = AssetToolsModule.Get().GetAssetTypeActionsForClass(AssetData.GetClass());

			if (AssetTypeActionsWeak.IsValid())
			{
				TArray<FAssetData> AssetArray = {AssetData};
				constexpr bool bIsPreview = false;

				// Check the selected assets, and filter out the ones we cannot Edit.
				// This might also create a "Save Content" window, e.g. if Editing implies loading a different Level and the current one is dirty 
				AssetArray = AssetTypeActionsWeak.Pin()->GetValidAssetsForPreviewOrEdit(AssetArray, bIsPreview);

				if (!AssetArray.IsEmpty())
				{
					AssetsToLoad.Append(AssetArray);
				}
			}
		}
	}

	// Open the editor(s)
	if (AssetsToLoad.Num() > 0)
	{
		FScopedSlowTask SlowTask(0, LOCTEXT("LoadingSelectedObject", "Editing assets..."));
		SlowTask.MakeDialogDelayed(.1f);
		UAssetEditorSubsystem* AssetEditorSubsystem = GEditor->GetEditorSubsystem<UAssetEditorSubsystem>();

		for (const FAssetData& AssetData : AssetsToLoad)
		{
			UObject* EditObject = AssetData.GetAsset();
			if (EditObject)
			{
				AssetEditorSubsystem->OpenEditorForAsset(EditObject);
			}
		}
	}
}

void FAssetManagerEditorModule::OnReferenceViewerTabClosed(TSharedRef<SDockTab> InClosedTab)
{
	FName TabID = FName(InClosedTab->GetLayoutIdentifier().ToString());

	uint32 PairToRemove = INDEX_NONE;
	for (const TPair<uint32, FName>& Pair : AssetsHashToTabID)
	{
		if (Pair.Value == TabID)
		{
			PairToRemove = Pair.Key;
			break;
		}
	}

	if (const TSharedPtr<SReferenceViewer>& ReferenceViewer = StaticCastSharedRef<SReferenceViewer>(InClosedTab->GetContent()))
	{
		ReferenceViewer->OnReferenceViewerSelectionChanged().RemoveAll(this);
	}

	AssetsHashToTabID.Remove(PairToRemove);
	ReferenceViewerTabs.Remove(FName(InClosedTab->GetLayoutIdentifier().ToString()));
}

bool FAssetManagerEditorModule::GetManagedPackageListForAssetData(const FAssetData& AssetData, TSet<FName>& ManagedPackageSet)
{
	InitializeRegistrySources(true);

	TSet<FPrimaryAssetId> PrimaryAssetSet;
	bool bFoundAny = false;
	int32 ChunkId = IAssetManagerEditorModule::ExtractChunkIdFromFakeAssetData(AssetData);
	if (ChunkId != INDEX_NONE)
	{
		// This is a chunk meta asset
		const FAssetManagerChunkInfo* FoundChunkInfo = CurrentRegistrySource->ChunkAssignments.Find(ChunkId);

		if (!FoundChunkInfo)
		{
			return false;
		}
		
		for (const FAssetIdentifier& AssetIdentifier : FoundChunkInfo->AllAssets)
		{
			if (AssetIdentifier.PackageName != NAME_None)
			{
				bFoundAny = true;
				ManagedPackageSet.Add(AssetIdentifier.PackageName);
			}
		}
	}
	else
	{
		FPrimaryAssetId FoundPrimaryAssetId = IAssetManagerEditorModule::ExtractPrimaryAssetIdFromFakeAssetData(AssetData);
		if (FoundPrimaryAssetId.IsValid())
		{
			// Primary asset meta package
			PrimaryAssetSet.Add(FoundPrimaryAssetId);
		}
		else
		{
			// Normal asset package
			FoundPrimaryAssetId = UAssetManager::Get().GetPrimaryAssetIdForData(AssetData);

			if (!FoundPrimaryAssetId.IsValid())
			{
				return false;
			}

			PrimaryAssetSet.Add(FoundPrimaryAssetId);
		}
	}

	TArray<FAssetIdentifier> FoundDependencies;

	if (CurrentRegistrySource->HasRegistry())
	{
		for (const FPrimaryAssetId& PrimaryAssetId : PrimaryAssetSet)
		{
			CurrentRegistrySource->GetDependencies(PrimaryAssetId, FoundDependencies, UE::AssetRegistry::EDependencyCategory::Manage);
		}
	}
	
	for (const FAssetIdentifier& Identifier : FoundDependencies)
	{
		if (Identifier.PackageName != NAME_None)
		{
			bFoundAny = true;
			ManagedPackageSet.Add(Identifier.PackageName);
		}
	}
	return bFoundAny;
}

bool FAssetManagerEditorModule::GetStringValueForCustomColumn(const FAssetData& AssetData, FName ColumnName, FString& OutValue, const FAssetManagerEditorRegistrySource* OverrideRegistrySource /*= nullptr*/)
{
	const FAssetManagerEditorRegistrySource* RegistrySource = (OverrideRegistrySource == nullptr) ? CurrentRegistrySource : OverrideRegistrySource;

	if (!RegistrySource || !RegistrySource->HasRegistry())
	{
		return false;
	}

	UAssetManager& AssetManager = UAssetManager::Get();

	if (ColumnName == ManagedResourceSizeName || ColumnName == ManagedDiskSizeName || ColumnName == DiskSizeName || ColumnName == TotalUsageName)
	{
		// Get integer, convert to string
		int64 IntegerValue = 0;
		if (GetIntegerValueForCustomColumn(AssetData, ColumnName, IntegerValue))
		{
			OutValue = LexToString(IntegerValue);
			return true;
		}
	}
	else if (ColumnName == CookRuleName)
	{
		EPrimaryAssetCookRule CookRule;

		CookRule = AssetManager.GetPackageCookRule(AssetData.PackageName);

		switch (CookRule)
		{
		case EPrimaryAssetCookRule::AlwaysCook: 
			OutValue = TEXT("Always");
			return true;
		case EPrimaryAssetCookRule::DevelopmentAlwaysProductionUnknownCook:
			OutValue = TEXT("DevelopmentAlwaysProductionUnknown");
			return true;
		case EPrimaryAssetCookRule::DevelopmentAlwaysProductionNeverCook:
			OutValue = TEXT("DevelopmentAlwaysProductionNever");
			return true;
		case EPrimaryAssetCookRule::ProductionNeverCook:
			OutValue = TEXT("ProductionNeverCook");
			return true;
		case EPrimaryAssetCookRule::NeverCook:
			OutValue = TEXT("Never");
			return true;
		}
	}
	else if (ColumnName == ChunksName)
	{
		TArray<int32> FoundChunks;
		OutValue.Reset();

		if (RegistrySource->bIsEditor)
		{
			// The in-memory data is wrong, ask the asset manager
			AssetManager.GetPackageChunkIds(AssetData.PackageName, RegistrySource->TargetPlatform, AssetData.GetChunkIDs(), FoundChunks);
		}
		else
		{
			FAssetData PlatformData = RegistrySource->GetAssetByObjectPath(AssetData.GetSoftObjectPath());
			if (PlatformData.IsValid())
			{
				FoundChunks = PlatformData.GetChunkIDs();
			}
		}
		
		FoundChunks.Sort();

		for (int32 Chunk : FoundChunks)
		{
			if (!OutValue.IsEmpty())
			{
				OutValue += TEXT("+");
			}
			OutValue += LexToString(Chunk);
		}
		return true;
	}
	else if (ColumnName == PluginName)
	{
		OutValue = FPackageName::SplitPackageNameRoot(AssetData.PackageName, nullptr);
		return OutValue.Len() > 0;
	}
	else
	{
		// Get base value of asset tag
		return RegistrySource->GetAssetTagByObjectPath(AssetData.GetSoftObjectPath(), ColumnName, OutValue);
	}

	return false;
}

bool FAssetManagerEditorModule::GetDisplayTextForCustomColumn(const FAssetData& AssetData, FName ColumnName, FText& OutValue, const FAssetManagerEditorRegistrySource* OverrideRegistrySource /*= nullptr*/)
{
	const FAssetManagerEditorRegistrySource* RegistrySource = (OverrideRegistrySource == nullptr) ? CurrentRegistrySource : OverrideRegistrySource;

	if (!RegistrySource || !RegistrySource->HasRegistry())
	{
		return false;
	}

	UAssetManager& AssetManager = UAssetManager::Get();

	if (ColumnName == ManagedResourceSizeName || ColumnName == ManagedDiskSizeName || ColumnName == DiskSizeName || ColumnName == TotalUsageName || ColumnName == UE::AssetRegistry::Stage_ChunkSizeFName || ColumnName == UE::AssetRegistry::Stage_ChunkCompressedSizeFName)
	{
		// Get integer, convert to string
		int64 IntegerValue = 0;
		if (GetIntegerValueForCustomColumn(AssetData, ColumnName, IntegerValue))
		{
			if (ColumnName == TotalUsageName)
			{
				OutValue = FText::AsNumber(IntegerValue);
			}
			else
			{
				// Display size properly
				OutValue = FText::AsMemory(IntegerValue);
			}
			return true;
		}
	}
	else if (ColumnName == CookRuleName)
	{
		EPrimaryAssetCookRule CookRule;

		CookRule = AssetManager.GetPackageCookRule(AssetData.PackageName);

		switch (CookRule)
		{
		case EPrimaryAssetCookRule::AlwaysCook:
			OutValue = LOCTEXT("AlwaysCook", "Always");
			return true;
		case EPrimaryAssetCookRule::DevelopmentAlwaysProductionUnknownCook:
			OutValue = LOCTEXT("DevelopmentAlwaysProductionUnknownCook", "DevelopmentAlwaysProductionUnknown");
			return true;
		case EPrimaryAssetCookRule::DevelopmentAlwaysProductionNeverCook:
			OutValue = LOCTEXT("DevelopmentAlwaysProductionNeverCook", "DevelopmentAlwaysProductionNever");
			return true;
		case EPrimaryAssetCookRule::ProductionNeverCook:
			OutValue = LOCTEXT("ProductionNeverCook", "ProductionNever");
			return true;
		case EPrimaryAssetCookRule::NeverCook:
			OutValue = LOCTEXT("NeverCook", "Never");
			return true;
		}
	}
	else if (ColumnName == ChunksName || ColumnName == PluginName)
	{
		FString OutString;

		if (GetStringValueForCustomColumn(AssetData, ColumnName, OutString))
		{
			OutValue = FText::AsCultureInvariant(OutString);
			return true;
		}
	}
	else
	{
		// Get base value of asset tag
		return RegistrySource->GetAssetTagByObjectPath(AssetData.GetSoftObjectPath(), ColumnName, OutValue);
	}

	return false;
}

bool FAssetManagerEditorModule::GetIntegerValueForCustomColumn(const FAssetData& AssetData, FName ColumnName, int64& OutValue, const FAssetManagerEditorRegistrySource* OverrideRegistrySource /*= nullptr*/)
{
	const FAssetManagerEditorRegistrySource* RegistrySource = (OverrideRegistrySource == nullptr) ? CurrentRegistrySource : OverrideRegistrySource;

	if (!RegistrySource || !RegistrySource->HasRegistry())
	{
		return false;
	}

	UAssetManager& AssetManager = UAssetManager::Get();

	if (ColumnName == ManagedResourceSizeName || ColumnName == ManagedDiskSizeName)
	{
		FName SizeTag = (ColumnName == ManagedResourceSizeName) ? ResourceSizeName : DiskSizeName;
		TSet<FName> AssetPackageSet;

		if (!GetManagedPackageListForAssetData(AssetData, AssetPackageSet))
		{
			// Just return exclusive
			return GetIntegerValueForCustomColumn(AssetData, SizeTag, OutValue);
		}

		int64 TotalSize = 0;
		bool bFoundAny = false;

		for (FName PackageName : AssetPackageSet)
		{
			TArray<FAssetData> FoundData;
			FARFilter AssetFilter;
			AssetFilter.PackageNames.Add(PackageName);
			AssetFilter.bIncludeOnlyOnDiskAssets = true;

			if (AssetRegistry->GetAssets(AssetFilter, FoundData) && FoundData.Num() > 0)
			{
				// Use first one
				FAssetData& ManagedAssetData = FoundData[0];

				int64 PackageSize = 0;
				if (GetIntegerValueForCustomColumn(ManagedAssetData, SizeTag, PackageSize))
				{
					bFoundAny = true;
					TotalSize += PackageSize;
				}
			}
		}

		if (bFoundAny)
		{
			OutValue = TotalSize;
			return true;
		}
	}
	else if (ColumnName == DiskSizeName)
	{
		// If the asset registry has the staged sizes written back, then use that as an accurate representation of the
		// actual chunks in the package.
		if (RegistrySource->GetAssetTagByObjectPath(AssetData.GetSoftObjectPath(), UE::AssetRegistry::Stage_ChunkSizeFName, OutValue))
		{
			return true;
		}

		// Otherwise, use the DiskSize generated by SavePackage.
		TOptional<FAssetPackageData> FoundData = RegistrySource->GetAssetPackageDataCopy(AssetData.PackageName);

		if (FoundData && FoundData->DiskSize >= 0)
		{
			OutValue = FoundData->DiskSize;
			return true;
		}
	}
	else if (ColumnName == ResourceSizeName)
	{
		// Resource size can currently only be calculated for loaded assets, so load and check
		UObject* Asset = AssetData.GetAsset();

		if (Asset)
		{
			OutValue = Asset->GetResourceSizeBytes(EResourceSizeMode::EstimatedTotal);
			return true;
		}
	}
	else if (ColumnName == TotalUsageName)
	{
		int64 TotalWeight = 0;

		TSet<FPrimaryAssetId> ReferencingPrimaryAssets;

		AssetManager.GetPackageManagers(AssetData.PackageName, false, ReferencingPrimaryAssets);

		for (const FPrimaryAssetId& PrimaryAssetId : ReferencingPrimaryAssets)
		{
			FPrimaryAssetRules Rules = AssetManager.GetPrimaryAssetRules(PrimaryAssetId);

			if (!Rules.IsDefault())
			{
				TotalWeight += Rules.Priority;
			}
		}

		OutValue = TotalWeight;
		return true;
	}
	else
	{
		// Get base value of asset tag
		return RegistrySource->GetAssetTagByObjectPath(AssetData.GetSoftObjectPath(), ColumnName, OutValue);
	}

	return false;
}

FString FAssetManagerEditorModule::GetSavedAssetRegistryPath(ITargetPlatform* TargetPlatform)
{
	if (!TargetPlatform)
	{
		return FString();
	}

	FString PlatformName = TargetPlatform->PlatformName();

	// Initialize sandbox wrapper
	if (!CookedSandbox)
	{
		CookedSandbox = FSandboxPlatformFile::Create(false);

		FString OutputDirectory = FPaths::Combine(*FPaths::ProjectDir(), TEXT("Saved"), TEXT("Cooked"), TEXT("[Platform]"));
		FPaths::NormalizeDirectoryName(OutputDirectory);

		CookedSandbox->Initialize(&FPlatformFileManager::Get().GetPlatformFile(), *FString::Printf(TEXT("-sandbox=\"%s\""), *OutputDirectory));
	}

	if (!EditorCookedSandbox)
	{
		EditorCookedSandbox = FSandboxPlatformFile::Create(false);

		FString OutputDirectory = FPaths::Combine(*FPaths::ProjectDir(), TEXT("Saved"), TEXT("EditorCooked"), TEXT("[Platform]"));
		FPaths::NormalizeDirectoryName(OutputDirectory);

		EditorCookedSandbox->Initialize(&FPlatformFileManager::Get().GetPlatformFile(), *FString::Printf(TEXT("-sandbox=\"%s\""), *OutputDirectory));
	}

	FString CommandLinePath;
	FParse::Value(FCommandLine::Get(), TEXT("AssetRegistryFile="), CommandLinePath);
	CommandLinePath.ReplaceInline(TEXT("[Platform]"), *PlatformName);
	
	// We can only load DevelopmentAssetRegistry, the normal asset registry doesn't have enough data to be useful
	FString CookedDevelopmentAssetRegistry = FPaths::ProjectDir() / TEXT("Metadata") / GetDevelopmentAssetRegistryFilename();

	FString DevCookedPath = CookedSandbox->ConvertToAbsolutePathForExternalAppForWrite(*CookedDevelopmentAssetRegistry).Replace(TEXT("[Platform]"), *PlatformName);
	FString DevEditorCookedPath = EditorCookedSandbox->ConvertToAbsolutePathForExternalAppForWrite(*CookedDevelopmentAssetRegistry).Replace(TEXT("[Platform]"), *PlatformName);
	FString DevSharedCookedPath = FPaths::Combine(*FPaths::ProjectSavedDir(), TEXT("SharedIterativeBuild"), PlatformName, TEXT("Metadata"), GetDevelopmentAssetRegistryFilename());

	// Try command line, then cooked, then shared build
	if (!CommandLinePath.IsEmpty() && IFileManager::Get().FileExists(*CommandLinePath))
	{
		return CommandLinePath;
	}

	if (IFileManager::Get().FileExists(*DevCookedPath))
	{
		return DevCookedPath;
	}

	if (IFileManager::Get().FileExists(*DevEditorCookedPath))
	{
		return DevEditorCookedPath;
	}

	if (IFileManager::Get().FileExists(*DevSharedCookedPath))
	{
		return DevSharedCookedPath;
	}

	return FString();
}

void FAssetManagerEditorModule::GetAvailableRegistrySources(TArray<const FAssetManagerEditorRegistrySource*>& AvailableSources)
{
	InitializeRegistrySources(false);

	for (const TPair<FString, FAssetManagerEditorRegistrySource>& Pair : RegistrySourceMap)
	{
		AvailableSources.Add(&Pair.Value);
	}
}

const FAssetManagerEditorRegistrySource* FAssetManagerEditorModule::GetCurrentRegistrySource(bool bNeedManagementData)
{
	InitializeRegistrySources(bNeedManagementData);

	return CurrentRegistrySource;
}

void FAssetManagerEditorRegistrySource::LoadRegistryTimestamp()
{
	FFileStatData TimeData = IFileManager::Get().GetStatData(*SourceFilename);
	FDateTime UseTime = FDateTime::MinValue();
	if (TimeData.bIsValid)
	{
		if (TimeData.ModificationTime != FDateTime::MinValue())
		{
			UseTime = TimeData.ModificationTime;
		}
		else
		{
			UseTime = TimeData.CreationTime;
		}
	}

	if (UseTime != FDateTime::MinValue())
	{
		//Turn UTC into local
		FTimespan UTCOffset = FDateTime::Now() - FDateTime::UtcNow();
		UseTime += UTCOffset;

		SourceTimestamp = UseTime.ToString(TEXT("%Y.%m.%d %h:%M %A"));
	}
}

bool FAssetManagerEditorModule::PopulateRegistrySource(FAssetManagerEditorRegistrySource* InOutRegistrySource, const FString* OptInFilePath/*= nullptr*/)
{
	if (InOutRegistrySource->SourceName == FAssetManagerEditorRegistrySource::CustomSourceName)
	{
		TArray<FString> OutFilenames;
		if (OptInFilePath != nullptr && IFileManager::Get().FileExists(**OptInFilePath))
		{
			OutFilenames.Add(*OptInFilePath);
		}
		else
		{
			IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get();
			const void* ParentWindowWindowHandle = FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr);
			const TCHAR* DevelopmentAssetRegistryFilename = GetDevelopmentAssetRegistryFilename();
			const FText Title = LOCTEXT("LoadAssetRegistry", "Load DevelopmentAssetRegistry");
			const FString FileTypes = FString::Printf(TEXT("%s|*.bin"), DevelopmentAssetRegistryFilename);

			DesktopPlatform->OpenFileDialog(
				ParentWindowWindowHandle,
				Title.ToString(),
				TEXT(""),
				DevelopmentAssetRegistryFilename,
				FileTypes,
				EFileDialogFlags::None,
				OutFilenames
			);
		}

		if (OutFilenames.Num() == 1)
		{
			InOutRegistrySource->SourceTimestamp = FString();

			if (InOutRegistrySource->HasRegistry())
			{
				InOutRegistrySource->ClearRegistry();
				InOutRegistrySource->ChunkAssignments.Reset();
				InOutRegistrySource->bManagementDataInitialized = false;
			}

			InOutRegistrySource->SourceFilename = OutFilenames[0];
		}
		else
		{
			return false;
		}
	}

	if (!InOutRegistrySource->HasRegistry() && !InOutRegistrySource->SourceFilename.IsEmpty() && !InOutRegistrySource->bIsEditor)
	{
		InOutRegistrySource->SourceTimestamp = FString();

		bool bLoaded = false;
		FArrayReader SerializedAssetData;
		if (FFileHelper::LoadFileToArray(SerializedAssetData, *InOutRegistrySource->SourceFilename))
		{
			InOutRegistrySource->LoadRegistryTimestamp();

			FAssetRegistryState* NewState = new FAssetRegistryState();
			FAssetRegistrySerializationOptions Options(UE::AssetRegistry::ESerializationTarget::ForDevelopment);

			NewState->Serialize(SerializedAssetData, Options);

			if (NewState->GetNumAssets() > 0)
			{
				bLoaded = true;
				InOutRegistrySource->SetRegistryState(NewState);
			}
		}
		if (!bLoaded)
		{
			FNotificationInfo Info(FText::Format(LOCTEXT("LoadRegistryFailed_FailedToLoad", "Failed to load asset registry from {0}!"), FText::FromString(InOutRegistrySource->SourceFilename)));
			Info.ExpireDuration = 10.0f;
			FSlateNotificationManager::Get().AddNotification(Info);
			return false;
		}
	}

	if (!InOutRegistrySource->bManagementDataInitialized && InOutRegistrySource->HasRegistry())
	{
		InOutRegistrySource->ChunkAssignments.Reset();
		if (InOutRegistrySource->bIsEditor)
		{
			// Load chunk list from asset manager
			InOutRegistrySource->ChunkAssignments = UAssetManager::Get().GetChunkManagementMap();
		}
		else
		{
			// Iterate assets and look for chunks
			const FAssetRegistryState* RegistryState = InOutRegistrySource->GetOwnedRegistryState();
			checkf(RegistryState, TEXT("Should be non-null because HasRegistry() && !bIsEditor"));

			RegistryState->EnumerateAllAssets([&RegistryState, &InOutRegistrySource, this](const FAssetData& AssetData)
				{
					const FAssetData::FChunkArrayView ChunkIDs = AssetData.GetChunkIDs();
					if (!ChunkIDs.IsEmpty())
					{
						TArray<FAssetIdentifier> ManagerAssets;
						RegistryState->GetReferencers(AssetData.PackageName, ManagerAssets, UE::AssetRegistry::EDependencyCategory::Manage);

						for (int32 ChunkId : ChunkIDs)
						{
							FPrimaryAssetId ChunkAssetId = UAssetManager::CreatePrimaryAssetIdFromChunkId(ChunkId);

							FAssetManagerChunkInfo* ChunkAssignmentSet = InOutRegistrySource->ChunkAssignments.Find(ChunkId);

							if (!ChunkAssignmentSet)
							{
								// First time found, read the graph
								ChunkAssignmentSet = &InOutRegistrySource->ChunkAssignments.Add(ChunkId);

								TArray<FAssetIdentifier> ManagedAssets;

								RegistryState->GetDependencies(ChunkAssetId, ManagedAssets, UE::AssetRegistry::EDependencyCategory::Manage);

								for (const FAssetIdentifier& ManagedAsset : ManagedAssets)
								{
									ChunkAssignmentSet->ExplicitAssets.Add(ManagedAsset);
								}
							}

							ChunkAssignmentSet->AllAssets.Add(AssetData.PackageName);

							// Check to see if this was added by a management reference, if not register as an explicit reference
							bool bAddedFromManager = false;
							for (const FAssetIdentifier& Manager : ManagerAssets)
							{
								if (ChunkAssignmentSet->ExplicitAssets.Find(Manager))
								{
									bAddedFromManager = true;
								}
							}

							if (!bAddedFromManager)
							{
								ChunkAssignmentSet->ExplicitAssets.Add(AssetData.PackageName);
							}
						}
					}
				});
		}

		InOutRegistrySource->bManagementDataInitialized = true;
	}

	return true;
}

void FAssetManagerEditorModule::SetCurrentRegistrySource(const FString& SourceName)
{
	InitializeRegistrySources(false);

	FAssetManagerEditorRegistrySource* NewSource = RegistrySourceMap.Find(SourceName);

	if (NewSource)
	{
		if (!PopulateRegistrySource(NewSource))
		{
			bool Success = PopulateRegistrySource(RegistrySourceMap.Find(FAssetManagerEditorRegistrySource::EditorSourceName));
			ensure(Success);
		}
		CurrentRegistrySource = NewSource;
	}
	else
	{
		FNotificationInfo Info(FText::Format(LOCTEXT("LoadRegistryFailed_MissingFile", "Can't find registry source {0}! Reverting to Editor."), FText::FromString(SourceName)));
		Info.ExpireDuration = 10.0f;
		FSlateNotificationManager::Get().AddNotification(Info);
		CurrentRegistrySource = RegistrySourceMap.Find(FAssetManagerEditorRegistrySource::EditorSourceName);
	}

	check(CurrentRegistrySource);
	
	// Refresh UI
	if (AssetAuditUI.IsValid())
	{
		AssetAuditUI.Pin()->SetCurrentRegistrySource(CurrentRegistrySource);
	}
	if (SizeMapUI.IsValid())
	{
		SizeMapUI.Pin()->SetCurrentRegistrySource(CurrentRegistrySource);
	}
	if (ReferenceViewerUI.IsValid())
	{
		ReferenceViewerUI.Pin()->SetCurrentRegistrySource(CurrentRegistrySource);
	}
}

void FAssetManagerEditorModule::RefreshRegistryData()
{
	UAssetManager::Get().UpdateManagementDatabase(EUpdateManagementDatabaseFlags::BuildChunkMap | EUpdateManagementDatabaseFlags::ForceRefresh);

	// Rescan registry sources, try to restore the current one
	const FString OldSourceName = CurrentRegistrySource ? CurrentRegistrySource->SourceName : FString();

	CurrentRegistrySource = nullptr;
	InitializeRegistrySources(false);

	if (!OldSourceName.IsEmpty())
	{
		SetCurrentRegistrySource(OldSourceName);
	}
}

bool FAssetManagerEditorModule::IsPackageInCurrentRegistrySource(FName PackageName)
{
	if (CurrentRegistrySource && CurrentRegistrySource->HasRegistry() && !CurrentRegistrySource->bIsEditor)
	{
		const FAssetRegistryState* RegistryState = CurrentRegistrySource->GetOwnedRegistryState();
		checkf(RegistryState, TEXT("Should be non-null because HasRegistry() && !bIsEditor"));
		const FAssetPackageData* FoundData = RegistryState->GetAssetPackageData(PackageName);

		if (!FoundData || FoundData->DiskSize < 0)
		{
			return false;
		}
	}

	// In editor, no packages are filtered
	return true;
}

bool FAssetManagerEditorModule::FilterAssetIdentifiersForCurrentRegistrySource(TArray<FAssetIdentifier>& AssetIdentifiers, const FAssetManagerDependencyQuery& DependencyQuery, bool bForwardDependency)
{
	bool bMadeChange = false;
	if (!CurrentRegistrySource || !CurrentRegistrySource->HasRegistry() || CurrentRegistrySource->bIsEditor)
	{
		return bMadeChange;
	}
	const FAssetRegistryState* RegistryState = CurrentRegistrySource->GetOwnedRegistryState();
	checkf(RegistryState, TEXT("Should be non-null because HasRegistry() && !bIsEditor"));

	for (int32 Index = 0; Index < AssetIdentifiers.Num(); Index++)
	{
		FName PackageName = AssetIdentifiers[Index].PackageName;

		if (PackageName != NAME_None)
		{
			if (!IsPackageInCurrentRegistrySource(PackageName))
			{
				// Remove bad package
				AssetIdentifiers.RemoveAt(Index);

				if (!DependencyQuery.IsNone())
				{
					// If this is a redirector replace with references
					TArray<FAssetData> Assets;
					AssetRegistry->GetAssetsByPackageName(PackageName, Assets, true);

					for (const FAssetData& Asset : Assets)
					{
						if (Asset.IsRedirector())
						{
							TArray<FAssetIdentifier> FoundReferences;

							if (bForwardDependency)
							{
								RegistryState->GetDependencies(PackageName, FoundReferences, DependencyQuery.Categories, DependencyQuery.Flags);
							}
							else
							{
								RegistryState->GetReferencers(PackageName, FoundReferences, DependencyQuery.Categories, DependencyQuery.Flags);
							}

							AssetIdentifiers.Insert(FoundReferences, Index);
							break;
						}
					}
				}

				// Need to redo this index, it was either removed or replaced
				Index--;
			}
		}
	}
	return bMadeChange;
}

void FAssetManagerEditorModule::InitializeRegistrySources(bool bNeedManagementData)
{
	if (CurrentRegistrySource == nullptr)
	{
		// Clear old list
		RegistrySourceMap.Reset();

		// Add Editor source
		{
			FAssetManagerEditorRegistrySource EditorSource;
			EditorSource.SourceName = FAssetManagerEditorRegistrySource::EditorSourceName;
			EditorSource.SetUseEditorAssetRegistry(AssetRegistry);

			RegistrySourceMap.Add(FAssetManagerEditorRegistrySource::EditorSourceName, MoveTemp(EditorSource));
		}

		TArray<ITargetPlatform*> Platforms = GetTargetPlatformManager()->GetTargetPlatforms();

		for (ITargetPlatform* CheckPlatform : Platforms)
		{
			FString RegistryPath = GetSavedAssetRegistryPath(CheckPlatform);

			if (!RegistryPath.IsEmpty())
			{
				FAssetManagerEditorRegistrySource PlatformSource;
				PlatformSource.SourceName = CheckPlatform->PlatformName();
				PlatformSource.SourceFilename = RegistryPath;
				PlatformSource.TargetPlatform = CheckPlatform;

				RegistrySourceMap.Add(PlatformSource.SourceName, MoveTemp(PlatformSource));
			}
		}

		// Add Custom source
		{
			FAssetManagerEditorRegistrySource CustomSource;
			CustomSource.SourceName = FAssetManagerEditorRegistrySource::CustomSourceName;

			RegistrySourceMap.Add(CustomSource.SourceName, MoveTemp(CustomSource));
		}

		// Select the Editor source by default
		CurrentRegistrySource = RegistrySourceMap.Find(FAssetManagerEditorRegistrySource::EditorSourceName);
	}
	check(CurrentRegistrySource);

	if (bNeedManagementData && !CurrentRegistrySource->bManagementDataInitialized)
	{
		SetCurrentRegistrySource(CurrentRegistrySource->SourceName);
	}
}

void FAssetManagerEditorModule::PerformAuditConsoleCommand(const TArray<FString>& Args)
{
	// Turn off as it makes diffing hard
	TGuardValue DisableLogTimes(GPrintLogTimes, ELogTimes::None);

	UAssetManager::Get().UpdateManagementDatabase();

	// Now print assets with multiple labels
	LogAssetsWithMultipleLabels();
}

bool FAssetManagerEditorModule::GetDependencyTypeArg(const FString& Arg, UE::AssetRegistry::EDependencyQuery& OutRequiredFlags)
{
	if (Arg.Compare(TEXT("-hardonly"), ESearchCase::IgnoreCase) == 0)
	{
		OutRequiredFlags = UE::AssetRegistry::EDependencyQuery::Hard;
		return true;
	}
	else if (Arg.Compare(TEXT("-softonly"), ESearchCase::IgnoreCase) == 0)
	{
		OutRequiredFlags = UE::AssetRegistry::EDependencyQuery::Soft;
		return true;
	}
	return false;
}

void FAssetManagerEditorModule::PerformDependencyChainConsoleCommand(const TArray<FString>& Args)
{
	if (Args.Num() < 2)
	{
		UE_LOGF(LogAssetManagerEditor, Display, "FindDepChain given incorrect number of arguments.  Usage: %ls", FindDepChainHelpText);
		return;
	}

	FName TargetPath = FName(*Args[0].ToLower());
	FName SearchRoot = FName(*Args[1].ToLower());

	UE::AssetRegistry::EDependencyQuery RequiredDependencyFlags = UE::AssetRegistry::EDependencyQuery::NoRequirements;
	if (Args.Num() > 2)
	{
		GetDependencyTypeArg(Args[2], RequiredDependencyFlags);
	}

	FindReferenceChains(TargetPath, SearchRoot, RequiredDependencyFlags);
}

void FAssetManagerEditorModule::PerformDependencyClassConsoleCommand(const TArray<FString>& Args)
{
	if (Args.Num() < 2)
	{
		UE_LOGF(LogAssetManagerEditor, Display, "FindDepClasses given incorrect number of arguments.  Usage: %ls", FindClassDepHelpText);
		return;
	}

	UE::AssetRegistry::EDependencyQuery RequiredDependencyFlags = UE::AssetRegistry::EDependencyQuery::NoRequirements;

	FName SourcePackagePath = FName(*Args[0].ToLower());
	TArray<FTopLevelAssetPath> TargetClasses;
	for (int32 i = 1; i < Args.Num(); ++i)
	{
		if (!GetDependencyTypeArg(Args[i], RequiredDependencyFlags))
		{
			if (!FPackageName::IsShortPackageName(Args[i]))
			{
				TargetClasses.AddUnique(FTopLevelAssetPath(Args[i]));
			}
			else
			{
				UE_LOGF(LogClass, Warning, "Short class names are not supported: %ls", *Args[i]);
			}
		}
	}

	TArray<FName> PackagesToSearch;

	//determine if the user passed us a package, or a directory
	TArray<FAssetData> PackageAssets;
	AssetRegistry->GetAssetsByPackageName(SourcePackagePath, PackageAssets);
	if (PackageAssets.Num() > 0)
	{
		PackagesToSearch.Add(SourcePackagePath);
	}
	else
	{
		TArray<FAssetData> AssetsInSearchPath;
		if (AssetRegistry->GetAssetsByPath(SourcePackagePath, /*inout*/ AssetsInSearchPath, /*bRecursive=*/ true))
		{
			for (const FAssetData& AssetData : AssetsInSearchPath)
			{
				PackagesToSearch.AddUnique(AssetData.PackageName);
			}
		}
	}

	for (FName SourcePackage : PackagesToSearch)
	{
		UE_LOGF(LogAssetManagerEditor, Verbose, "FindDepClasses for: %ls", *SourcePackage.ToString());
		FindClassDependencies(SourcePackage, TargetClasses, RequiredDependencyFlags);
	}
}

bool FAssetManagerEditorModule::GetPackageDependencyChain(FName SourcePackage, FName TargetPackage, TArray<FName>& VisitedPackages, TArray<FName>& OutDependencyChain, UE::AssetRegistry::EDependencyQuery RequiredFlags)
{
	//avoid crashing from circular dependencies.
	if (VisitedPackages.Contains(SourcePackage))
	{
		return false;
	}
	VisitedPackages.AddUnique(SourcePackage);

	if (SourcePackage == TargetPackage)
	{
		OutDependencyChain.Add(SourcePackage);
		return true;
	}

	TArray<FName> SourceDependencies;
	if (AssetRegistry->GetDependencies(SourcePackage, SourceDependencies, UE::AssetRegistry::EDependencyCategory::Package, RequiredFlags) == false)
	{
		return false;
	}

	int32 DependencyCounter = 0;
	while (DependencyCounter < SourceDependencies.Num())
	{
		const FName& ChildPackageName = SourceDependencies[DependencyCounter];
		if (GetPackageDependencyChain(ChildPackageName, TargetPackage, VisitedPackages, OutDependencyChain, RequiredFlags))
		{
			OutDependencyChain.Add(SourcePackage);
			return true;
		}
		++DependencyCounter;
	}

	return false;
}

void FAssetManagerEditorModule::GetPackageDependenciesPerClass(FName SourcePackage, const TArray<FTopLevelAssetPath>& TargetClasses, TArray<FName>& VisitedPackages, TArray<FName>& OutDependentPackages, UE::AssetRegistry::EDependencyQuery RequiredDependencyFlags)
{
	//avoid crashing from circular dependencies.
	if (VisitedPackages.Contains(SourcePackage))
	{
		return;
	}
	VisitedPackages.AddUnique(SourcePackage);

	TArray<FName> SourceDependencies;
	if (AssetRegistry->GetDependencies(SourcePackage, SourceDependencies, UE::AssetRegistry::EDependencyCategory::Package, RequiredDependencyFlags) == false)
	{
		return;
	}

	int32 DependencyCounter = 0;
	while (DependencyCounter < SourceDependencies.Num())
	{
		const FName& ChildPackageName = SourceDependencies[DependencyCounter];
		GetPackageDependenciesPerClass(ChildPackageName, TargetClasses, VisitedPackages, OutDependentPackages, RequiredDependencyFlags);
		++DependencyCounter;
	}

	FARFilter Filter;
	Filter.PackageNames.Add(SourcePackage);
	Filter.ClassPaths = TargetClasses;
	Filter.bIncludeOnlyOnDiskAssets = true;

	TArray<FAssetData> PackageAssets;
	if (AssetRegistry->GetAssets(Filter, PackageAssets))
	{
		if (PackageAssets.Num() > 0)
		{
			OutDependentPackages.AddUnique(SourcePackage);
		}
	}
}

void FAssetManagerEditorModule::FindReferenceChains(FName TargetPackageName, FName RootSearchPath, UE::AssetRegistry::EDependencyQuery RequiredDependencyFlags)
{
	//find all the assets we think might depend on our target through some chain
	TArray<FAssetData> AssetsInSearchPath;
	AssetRegistry->GetAssetsByPath(RootSearchPath, /*inout*/ AssetsInSearchPath, /*bRecursive=*/ true);

	//consolidate assets into a unique set of packages for dependency searching. reduces redundant work.
	TArray<FName> SearchPackages;
	for (const FAssetData& AD : AssetsInSearchPath)
	{
		SearchPackages.AddUnique(AD.PackageName);
	}

	int32 CurrentFoundChain = 0;
	TArray<TArray<FName>> FoundChains;
	FoundChains.AddDefaulted(1);

	//try to find a dependency chain that links each of these packages to our target.
	TArray<FName> VisitedPackages;
	for (const FName& SearchPackage : SearchPackages)
	{
		VisitedPackages.Reset();
		if (GetPackageDependencyChain(SearchPackage, TargetPackageName, VisitedPackages, FoundChains[CurrentFoundChain], RequiredDependencyFlags))
		{
			++CurrentFoundChain;
			FoundChains.AddDefaulted(1);
		}
	}

	UE_LOGF(LogAssetManagerEditor, Log, "Found %i, Dependency Chains to %ls from directory %ls", CurrentFoundChain, *TargetPackageName.ToString(), *RootSearchPath.ToString());
	for (int32 ChainIndex = 0; ChainIndex < CurrentFoundChain; ++ChainIndex)
	{
		TArray<FName>& FoundChain = FoundChains[ChainIndex];
		UE_LOGF(LogAssetManagerEditor, Log, "Chain %i", ChainIndex);

		for (FName& Name : FoundChain)
		{
			UE_LOGF(LogAssetManagerEditor, Log, "\t%ls", *Name.ToString());
		}
	}
}

void FAssetManagerEditorModule::FindClassDependencies(FName SourcePackageName, const TArray<FTopLevelAssetPath>& TargetClasses, UE::AssetRegistry::EDependencyQuery RequiredDependencyFlags)
{
	TArray<FAssetData> PackageAssets;
	if (!AssetRegistry->GetAssetsByPackageName(SourcePackageName, PackageAssets))
	{
		UE_LOGF(LogAssetManagerEditor, Log, "Couldn't find source package %ls. Abandoning class dep search.  ", *SourcePackageName.ToString());
		return;
	}

	TArray<FName> VisitedPackages;
	TArray<FName> DependencyPackages;
	GetPackageDependenciesPerClass(SourcePackageName, TargetClasses, VisitedPackages, DependencyPackages, RequiredDependencyFlags);

	if (DependencyPackages.Num() > 0)
	{
		UE_LOGF(LogAssetManagerEditor, Log, "Found %i: dependencies for %ls of the target classes", DependencyPackages.Num(), *SourcePackageName.ToString());
		for (FName DependencyPackage : DependencyPackages)
		{
			UE_LOGF(LogAssetManagerEditor, Log, "\t%ls", *DependencyPackage.ToString());
		}

		for (FName DependencyPackage : DependencyPackages)
		{
			TArray<FName> Chain;
			VisitedPackages.Reset();
			GetPackageDependencyChain(SourcePackageName, DependencyPackage, VisitedPackages, Chain, RequiredDependencyFlags);

			UE_LOGF(LogAssetManagerEditor, Log, "Chain to package: %ls", *DependencyPackage.ToString());
			TArray<FAssetData> DepAssets;

			FARFilter Filter;
			Filter.PackageNames.Add(DependencyPackage);
			Filter.ClassPaths = TargetClasses;
			Filter.bIncludeOnlyOnDiskAssets = true;

			if (AssetRegistry->GetAssets(Filter, DepAssets))
			{
				for (const FAssetData& DepAsset : DepAssets)
				{
					if (TargetClasses.Contains(DepAsset.AssetClassPath))
					{
						UE_LOGF(LogAssetManagerEditor, Log, "Asset: %ls class: %ls", *DepAsset.AssetName.ToString(), *DepAsset.AssetClassPath.ToString());
					}
				}
			}

			for (FName DepChainEntry : Chain)
			{
				UE_LOGF(LogAssetManagerEditor, Log, "\t%ls", *DepChainEntry.ToString());
			}
		}
	}
}

void FAssetManagerEditorModule::CopyPackageReferences(const TArray<FAssetData> SelectedPackages)
{
	TArray<FAssetData> SortedItems = SelectedPackages;
	SortedItems.Sort([](const FAssetData& One, const FAssetData Two)
		{
			return One.PackagePath.Compare(Two.PackagePath) < 0;
		});

	FString ClipboardText = FString::JoinBy(SortedItems, LINE_TERMINATOR, [](const FAssetData& Item)
		{
			return Item.GetExportTextName();
		});

	FPlatformApplicationMisc::ClipboardCopy(*ClipboardText);
}

void FAssetManagerEditorModule::CopyPackagePaths(const TArray<FAssetData> SelectedPackages)
{
	TArray<FAssetData> SortedItems = SelectedPackages;
	SortedItems.Sort([](const FAssetData& One, const FAssetData Two)
		{
			return One.PackagePath.Compare(Two.PackagePath) < 0;
		});

	FString ClipboardText = FString::JoinBy(SortedItems, LINE_TERMINATOR, [](const FAssetData& Item)
		{
			FString ItemFilename = FPackageName::LongPackageNameToFilename(Item.PackageName.ToString(), FPackageName::GetAssetPackageExtension());

			if (FPaths::FileExists(ItemFilename))
			{
				ItemFilename = FPaths::ConvertRelativePathToFull(ItemFilename);
				FPaths::MakePlatformFilename(ItemFilename);
				return ItemFilename;
			}
			else
			{
				// Add a message for when a user tries to copy the path to a file that doesn't exist on disk of the form
				// <ItemName>: No file on disk
				return FString::Printf(TEXT("%s: No file on disk"), *Item.AssetName.ToString());
			}
		});

	FPlatformApplicationMisc::ClipboardCopy(*ClipboardText);
}

void FAssetManagerEditorModule::WriteProfileFile(const FString& Extension, const FString& FileContents)
{
	const FString PathName = *(FPaths::ProfilingDir() + TEXT("AssetAudit/"));
	IFileManager::Get().MakeDirectory(*PathName);

	const FString Filename = CreateProfileFilename(Extension, true);
	const FString FilenameFull = PathName + Filename;

	UE_LOGF(LogAssetManagerEditor, Log, "Saving %ls", *FPaths::ConvertRelativePathToFull(FilenameFull));
	FFileHelper::SaveStringToFile(FileContents, *FilenameFull);
}


void FAssetManagerEditorModule::LogAssetsWithMultipleLabels()
{
	UAssetManager& Manager = UAssetManager::Get();

	TMap<FName, TArray<FPrimaryAssetId>> PackageToLabelMap;
	TArray<FPrimaryAssetId> LabelNames;

	Manager.GetPrimaryAssetIdList(UAssetManager::PrimaryAssetLabelType, LabelNames);

	for (const FPrimaryAssetId& Label : LabelNames)
	{
		TArray<FName> LabeledPackages;

		Manager.GetManagedPackageList(Label, LabeledPackages);

		for (FName Package : LabeledPackages)
		{
			PackageToLabelMap.FindOrAdd(Package).AddUnique(Label);
		}
	}

	PackageToLabelMap.KeySort(FNameLexicalLess());

	UE_LOGF(LogAssetManagerEditor, Log, "\nAssets with multiple labels follow");

	// Print them out
	for (TPair<FName, TArray<FPrimaryAssetId>> Pair : PackageToLabelMap)
	{
		if (Pair.Value.Num() > 1)
		{
			FString TagString;
			for (const FPrimaryAssetId& Label : Pair.Value)
			{
				if (!TagString.IsEmpty())
				{
					TagString += TEXT(", ");
				}
				TagString += Label.ToString();
			}

			UE_LOGF(LogAssetManagerEditor, Log, "%ls has %ls", *Pair.Key.ToString(), *TagString);
		}		
	}
}


void FAssetManagerEditorModule::DumpAssetRegistry(const TArray<FString>& Args)
{
#if ASSET_REGISTRY_STATE_DUMPING_ENABLED
	UAssetManager& Manager = UAssetManager::Get();
	TArray<FString> ReportLines;

	const IAssetRegistry& LocalAssetRegistry = Manager.GetAssetRegistry();
	LocalAssetRegistry.DumpState(Args, ReportLines);

	Manager.WriteCustomReport(FString::Printf(TEXT("AssetRegistryState-%s.txt"), *FDateTime::Now().ToString()), ReportLines);
#endif
}


void FAssetManagerEditorModule::DumpAssetDependencies(const TArray<FString>& Args)
{
	check(UAssetManager::IsInitialized());

	UAssetManager& Manager = UAssetManager::Get();
	TArray<FPrimaryAssetTypeInfo> TypeInfos;

	Manager.UpdateManagementDatabase();

	Manager.GetPrimaryAssetTypeInfoList(TypeInfos);

	TypeInfos.Sort([](const FPrimaryAssetTypeInfo& LHS, const FPrimaryAssetTypeInfo& RHS) { return LHS.PrimaryAssetType.LexicalLess(RHS.PrimaryAssetType); });

	UE_LOGF(LogAssetManagerEditor, Log, "=========== Asset Manager Dependencies ===========");

	TArray<FString> ReportLines;

	ReportLines.Add(TEXT("digraph { "));

	for (const FPrimaryAssetTypeInfo& TypeInfo : TypeInfos)
	{
		struct FDependencyInfo
		{
			FName AssetName;
			FString AssetListString;

			FDependencyInfo(FName InAssetName, const FString& InAssetListString) : AssetName(InAssetName), AssetListString(InAssetListString) {}
		};

		TArray<FDependencyInfo> DependencyInfos;
		TArray<FPrimaryAssetId> PrimaryAssetIds;

		Manager.GetPrimaryAssetIdList(TypeInfo.PrimaryAssetType, PrimaryAssetIds);

		for (const FPrimaryAssetId& PrimaryAssetId : PrimaryAssetIds)
		{
			TArray<FAssetIdentifier> FoundDependencies;
			TArray<FString> DependencyStrings;

			AssetRegistry->GetDependencies(PrimaryAssetId, FoundDependencies, UE::AssetRegistry::EDependencyCategory::Manage);

			for (const FAssetIdentifier& Identifier : FoundDependencies)
			{
				FString ReferenceString = Identifier.ToString();
				DependencyStrings.Add(ReferenceString);

				ReportLines.Add(FString::Printf(TEXT("\t\"%s\" -> \"%s\";"), *PrimaryAssetId.ToString(), *ReferenceString));
			}

			DependencyStrings.Sort();

			DependencyInfos.Emplace(PrimaryAssetId.PrimaryAssetName, *FString::Join(DependencyStrings, TEXT(", ")));
		}

		if (DependencyInfos.Num() > 0)
		{
			UE_LOGF(LogAssetManagerEditor, Log, "  Type %ls:", *TypeInfo.PrimaryAssetType.ToString());

			DependencyInfos.Sort([](const FDependencyInfo& LHS, const FDependencyInfo& RHS) { return LHS.AssetName.LexicalLess(RHS.AssetName); });

			for (FDependencyInfo& DependencyInfo : DependencyInfos)
			{
				UE_LOGF(LogAssetManagerEditor, Log, "    %ls: depends on %ls", *DependencyInfo.AssetName.ToString(), *DependencyInfo.AssetListString);
			}
		}
	}

	ReportLines.Add(TEXT("}"));

	Manager.WriteCustomReport(FString::Printf(TEXT("PrimaryAssetReferences%s.gv"), *FDateTime::Now().ToString()), ReportLines);
}

bool FAssetManagerEditorModule::CreateOrEmptyCollection(ICollectionContainer& CollectionContainer, FName CollectionName, ECollectionShareType::Type ShareType, FText& OutError)
{
	if (CollectionContainer.CollectionExists(CollectionName, ShareType))
	{
		return CollectionContainer.EmptyCollection(CollectionName, ShareType, &OutError);
	}
	else if (CollectionContainer.CreateCollection(CollectionName, ShareType, ECollectionStorageMode::Static, &OutError))
	{
		return true;
	}

	return false;
}

bool FAssetManagerEditorModule::WriteCollection(FName CollectionName, ECollectionShareType::Type ShareType, const TArray<FName>& PackageNames, bool bShowFeedback)
{
	return WriteCollection(*FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(), CollectionName, ShareType, PackageNames, bShowFeedback);
}

bool FAssetManagerEditorModule::WriteCollection(ICollectionContainer& CollectionContainer, FName CollectionName, ECollectionShareType::Type ShareType, const TArray<FName>& PackageNames, bool bShowFeedback)
{
	FText ResultsMessage;
	bool bSuccess = false;

	TSet<FSoftObjectPath> ObjectPathsToAddToCollection;

	FARFilter Filter;
	Filter.PackageNames = PackageNames;
	Filter.bIncludeOnlyOnDiskAssets = true;
	TArray<FAssetData> AssetsInPackages;
	AssetRegistry->GetAssets(Filter, AssetsInPackages);
	for (const FAssetData& AssetData : AssetsInPackages)
	{
		ObjectPathsToAddToCollection.Add(AssetData.GetSoftObjectPath());
	}

	FText Error;
	if (ObjectPathsToAddToCollection.Num() == 0)
	{
		UE_LOGF(LogAssetManagerEditor, Log, "Nothing to add to collection %ls", *CollectionName.ToString());
		ResultsMessage = FText::Format(LOCTEXT("NothingToAddToCollection", "Nothing to add to collection {0}"), FText::FromName(CollectionName));
	}
	else if (CreateOrEmptyCollection(CollectionContainer, CollectionName, ShareType, Error))
	{
		if (CollectionContainer.AddToCollection(
				CollectionName, ShareType, ObjectPathsToAddToCollection.Array(), nullptr, &Error))
		{
			UE_LOGF(LogAssetManagerEditor, Log, "Updated collection %ls", *CollectionName.ToString());
			ResultsMessage = FText::Format(LOCTEXT("CreateCollectionSucceeded", "Updated collection {0}"), FText::FromName(CollectionName));
			bSuccess = true;
		}
		else
		{
			UE_LOGF(LogAssetManagerEditor, Warning, "Failed to update collection %ls. %ls", *CollectionName.ToString(), *Error.ToString());
			ResultsMessage = FText::Format(LOCTEXT("AddToCollectionFailed", "Failed to add to collection {0}. {1}"), FText::FromName(CollectionName), Error);
		}
	}
	else
	{
		UE_LOGF(LogAssetManagerEditor, Warning, "Failed to create collection %ls. %ls", *CollectionName.ToString(), *Error.ToString());
		ResultsMessage = FText::Format(LOCTEXT("CreateCollectionFailed", "Failed to create collection {0}. {1}"), FText::FromName(CollectionName), Error);
	}

	if (bShowFeedback)
	{
		FMessageDialog::Open(EAppMsgType::Ok, ResultsMessage);
	}

	return bSuccess;
}

void FAssetManagerEditorModule::AppendRegisteredCommandsToCommandList(TSharedRef<FUICommandList> ListToAppendTo)
{
	// we maintain a separate commands list so we can register these to the menu in the Reference Viewer Schema
	RegisteredCommands.Broadcast(ListToAppendTo, AdditionalCommandsInfoList);
}

TSharedRef<SWidget> FAssetManagerEditorModule::CreateAssetSourceControlContextMenu(const TArray<FAssetData>& SelectedAssets)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	//Find or add the tool menu extension for this case
	UToolMenu* ContextMenu = ToolMenus->ExtendMenu("GenericContextMenu.AssetSourceControl");
	
	//Find or add the section. We need at least one section in the context menu. We do not need any label since there is only one section.
	FToolMenuSection& Section = ContextMenu->FindOrAddSection("Section");

	//If we did not already add a UContentBrowserAssetContextMenuContext context object we must create one and add it.
	UContentBrowserAssetContextMenuContext* ContentBrowserAssetContextMenuContext = Section.Context.FindContext<UContentBrowserAssetContextMenuContext>();
	if(!ContentBrowserAssetContextMenuContext)
	{
		//Create a content browser asset context menu context. This is required to use the built in source control context menu.
		//The goal of this context is to pass the asset list on which we want to do source control operation.
		ContentBrowserAssetContextMenuContext = NewObject<UContentBrowserAssetContextMenuContext>();
		ContentBrowserAssetContextMenuContext->bCanBeModified = true;
		//Add the context to the section
		Section.Context.AddObject(ContentBrowserAssetContextMenuContext);
	}

	//Set the selected assets
	if(ensure(ContentBrowserAssetContextMenuContext))
	{
		ContentBrowserAssetContextMenuContext->SelectedAssets = SelectedAssets;
	}

	//Build the generic source control context menu
	AssetSourceControlContextMenu.MakeContextMenu(Section);

	//Generate the widget use to show the menu
	return ToolMenus->GenerateWidget(ContextMenu);
}

#undef LOCTEXT_NAMESPACE