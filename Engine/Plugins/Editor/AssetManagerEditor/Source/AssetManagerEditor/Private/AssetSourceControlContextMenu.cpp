// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetSourceControlContextMenu.h"

#include "AssetDefinition.h"
#include "AssetDefinitionRegistry.h"
#include "AssetRegistry/ARFilter.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "AssetViewUtils.h"
#include "Async/Async.h"
#include "ContentBrowserAssetDataCore.h"
#include "ContentBrowserAssetDataPayload.h"
#include "ContentBrowserDataMenuContexts.h"
#include "ContentBrowserMenuContexts.h"
#include "ContentBrowserModule.h"
#include "FileHelpers.h"
#include "HAL/IConsoleManager.h"
#include "HAL/PlatformApplicationMisc.h"
#include "HAL/PlatformFile.h"
#include "HAL/PlatformFileManager.h"
#include "IAssetTools.h"
#include "IContentBrowserSingleton.h"
#include "ISourceControlModule.h"
#include "ISourceControlRevision.h"
#include "Misc/MessageDialog.h"
#include "Misc/PackageName.h"
#include "SourceControlHelpers.h"
#include "SourceControlOperations.h"
#include "SourceControlWindows.h"
#include "RevisionControlStyle/RevisionControlStyle.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "Toolkits/AssetEditorToolkitMenuContext.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "UncontrolledChangelistsModule.h"

#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Images/SThrobber.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "AssetSourceControlContextMenu"

namespace UE::AssetSourceControlContextMenu::Private
{
	FText LargeOperationTitle = LOCTEXT("SCCLargeOperationWarningTitle", "Continue Operation?");

	DECLARE_DELEGATE_RetVal(bool, FIsAsyncProcessingActive);

	FNewToolMenuCustomWidget MakeCustomWidgetDelegate(const TAttribute<FText>& Label, const TAttribute<FSlateIcon>& Icon, const FIsAsyncProcessingActive& IsAsyncProcessingActive)
	{
		return FNewToolMenuCustomWidget::CreateLambda([Label, Icon, IsAsyncProcessingActive](const FToolMenuContext& InContext, const FToolMenuCustomWidgetContext& WidgetContext)
			{
				return SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(FMargin(2, 0, 0, 0))
					.AutoWidth()
					[
						SNew(SImage)
						.ColorAndOpacity( FSlateColor::UseSubduedForeground() )
						.Image(TAttribute<const FSlateBrush*>::CreateLambda([Icon]()
							{
								return Icon.IsSet() ? Icon.Get().GetIcon() : nullptr;
							}))
					]
				+ SHorizontalBox::Slot()
					.VAlign(VAlign_Center)
					.Padding(FMargin(8, 0, 0, 0))
					.AutoWidth()
					[
						SNew(STextBlock)
						.Text(Label)
						.TextStyle(WidgetContext.StyleSet, ISlateStyle::Join(WidgetContext.StyleName, ".Label"))
					]
				+ SHorizontalBox::Slot()
					.Padding(FMargin(8, 0, 0, 0))
					.VAlign(VAlign_Center)
					.AutoWidth()
					[
						SNew(SThrobber)
						.PieceImage(WidgetContext.StyleSet->GetBrush("Throbber.CircleChunk"))
						.Visibility_Lambda([IsAsyncProcessingActive]()
						{
							if (IsAsyncProcessingActive.Execute())
							{
								return EVisibility::Visible;
							}
							return EVisibility::Hidden;
						})
					];
			});
	};


	FToolMenuEntry& AddAsyncMenuEntry(
		FToolMenuSection& Section,
		FName Name,
		const TAttribute<FText>& Label,
		const TAttribute<FText>& ToolTip,
		const TAttribute<FSlateIcon>& Icon,
		const FToolUIActionChoice& Action,
		const FIsAsyncProcessingActive& IsAsyncProcessingActive
	)
	{
		FToolMenuEntry MenuEntry = FToolMenuEntry::InitMenuEntry(
			Name,
			Label,
			ToolTip,
			Icon,
			Action
		);

		if (IsAsyncProcessingActive.IsBound())
		{
			MenuEntry.MakeCustomWidget = MakeCustomWidgetDelegate(Label, Icon, IsAsyncProcessingActive);
		}
		return Section.AddEntry(MenuEntry);
	}
}

class FAssetSourceControlContextMenuState : public TSharedFromThis<FAssetSourceControlContextMenuState>
{
public:
	void Initialize(FToolMenuSection& InSection);

	bool IsValid() const;

	void Close();

private:
	bool AddSourceControlMenuOptions(FToolMenuSection& InSection);
	void FillSourceControlSubMenu(UToolMenu* Menu);
	void FillDiffAgainstSubMenu(UToolMenu* Menu);

	// Todo: Make TriggerCheckAtNumActions default value into a user option

	// Warn user if the number of Affected assets is greater than the limit that could affect performance
	// Use this if the operation works on files only
	FExecuteAction ExecutionCheck(FExecuteAction&& InAction, int32 TriggerCheckAtNumActions = 10) const;
	
	// Warn user if the number of Selected assets is greater than the limit that could affect performance
	// Use this if the operation can work on files or folders so it only needs the user selected items
	FExecuteAction ExecutionCheckSelectionOnly(FExecuteAction&& InAction, int32 TriggerCheckAtNumActions = 10) const;

	void ExecuteDiffSelected() const;
	void ExecuteSCCMerge() const;
	void ExecuteSCCCheckOut() const;
	void ExecuteSCCSyncAndCheckOut() const;
	void ExecuteSCCMakeWritable() const;
	void ExecuteSCCOpenForAdd() const;
	void ExecuteSCCCheckIn() const;
	void ExecuteSCCHistory() const;
	void ExecuteSCCDiffAgainstDepot() const;
	void ExecuteSCCRevert() const;
	void ExecuteSCCRevertWritable() const;
	void ExecuteSCCSync() const;
	void ExecuteSCCRefresh() const;
	void ExecuteSCCCopyPaths() const;

	bool CanExecuteSCCMerge() const;
	bool CanExecuteSCCCheckOut() const;
	bool CanExecuteSCCSyncAndCheckOut() const;
	bool CanExecuteSCCMakeWritable() const;
	bool CanExecuteSCCOpenForAdd() const;
	bool CanExecuteSCCCheckIn() const;
	bool CanExecuteSCCHistory() const;
	bool CanExecuteSCCRevert() const;
	bool CanExecuteSCCRevertWritable() const;
	bool CanExecuteSCCSync() const;
	bool CanExecuteSCCCopyPaths() const;
	bool CanExecuteSCCDiffAgainstDepot() const;
	bool CanExecuteDiffSelected() const;

	bool AllowExecuteSCCRevertUnsaved() const;

	/** Helper function to gather the package names of all selected assets */
	void GetSelectedPackageNames(TArray<FString>& OutPackageNames) const;

	/** Helper function to gather the packages containing all selected assets */
	void GetSelectedPackages(TArray<UPackage*>& OutPackages) const;

private:

	enum class EAsyncState : uint8
	{
		None,
		TryCacheCanExecute,
		SCCUpdate,
		Idle,
		Error,
		Cancelled,
	};

	bool IsActionEnabled(bool bEnabled) const;

	bool IsStillScanning(bool bEnabled) const;

	bool TrySetAsyncState(EAsyncState FromState, EAsyncState ToState);

	/** Initializes some variable used to in "CanExecute" checks that won't change at runtime or are too expensive to check every frame. */
	void NextAsyncState();

	/** Callback from a perforce update call. */
	void CacheCanExecuteVarsSCCHandler(const FSourceControlOperationRef& InOperation, ECommandResult::Type Result);

	/** Try using perforce cache to update state */
	void TryCacheCanExecuteVars(const TArray<FString>& InSelectedPaths, TArray<FString>* OptionalPathsWithUnknownState);

private:

	TArray<FAssetData> SelectedAssets; // Expanded list of assets, no folders
	TArray<FName> SelectedFolders; // Originating folders that SelectedAssets may have been expanded from
	TArray<FString> PathsWithUnknownState;
	TArray<FString> CheckedOutUsers;
	FText CheckedOutUsersText;

	int32 SelectedFileCount = 0; // Number of files that were user selected, SelectedAssets may be large after recursively expanding SelectedFolders

	TSharedPtr<class ISourceControlOperation, ESPMode::ThreadSafe> SCCOperation;
	std::atomic<EAsyncState> AsyncState = EAsyncState::None;

	// If folders are present in the selection then we adjust some behaviors to account for the potentially large amount of files to scan
	bool bContainsFolders = false;

	/** Cached CanExecute vars */
	bool bCanExecuteSCCMerge = false;
	bool bCanExecuteSCCCheckOut = false;
	bool bCanExecuteSCCSyncAndCheckOut = false;
	bool bCanExecuteSCCMakeWritable = false;
	bool bCanExecuteSCCOpenForAdd = false;
	bool bCanExecuteSCCCheckIn = false;
	bool bCanExecuteSCCHistory = false;
	bool bCanExecuteSCCRevert = false;
	bool bCanExecuteSCCSync = false;
	bool bCanExecuteSCCRevertWritable = false;
	bool bCanExecuteSCCCopyPaths = false;
};

FAssetSourceControlContextMenu::~FAssetSourceControlContextMenu()
{
	if (InnerState)
	{
		InnerState->Close();
	}
}

void FAssetSourceControlContextMenu::MakeContextMenu(FToolMenuSection& InSection)
{
	if (InnerState)
	{
		InnerState->Close();
	}

	InnerState = MakeShareable(new FAssetSourceControlContextMenuState());
	InnerState->Initialize(InSection);
}

void FAssetSourceControlContextMenuState::Initialize(FToolMenuSection& InSection)
{
	if (const UContentBrowserAssetContextMenuContext* Context = InSection.FindContext<UContentBrowserAssetContextMenuContext>())
	{
		if (Context->bCanBeModified)
		{
			SelectedAssets += Context->SelectedAssets;
		}
	}

	if (const UAssetEditorToolkitMenuContext* MenuContext = InSection.FindContext<UAssetEditorToolkitMenuContext>())
	{
		if (MenuContext->Toolkit.IsValid() && MenuContext->Toolkit.Pin()->IsActuallyAnAsset())
		{
			for (const UObject* EditedAsset : *MenuContext->Toolkit.Pin()->GetObjectsCurrentlyBeingEdited())
			{
				SelectedAssets.Add(FAssetData(EditedAsset));
			}
		}
	}

	SelectedFileCount = SelectedAssets.Num();

	if (const UContentBrowserDataMenuContext_FolderMenu* ContextObject = InSection.FindContext<UContentBrowserDataMenuContext_FolderMenu>())
	{
		// Form a filter from the paths
		FARFilter Filter;
		Filter.bRecursivePaths = true;

		for (const FContentBrowserItem& SelectedItem : ContextObject->SelectedItems)
		{
			const FContentBrowserItemData* SelectedItemData = SelectedItem.GetPrimaryInternalItem();
			if (!SelectedItemData)
			{
				continue;
			}

			const UContentBrowserDataSource* DataSource = SelectedItemData->GetOwnerDataSource();
			if (!DataSource)
			{
				continue;
			}

			for (const FContentBrowserItemData& InternalItem : SelectedItem.GetInternalItems())
			{
				if (TSharedPtr<const FContentBrowserAssetFolderItemDataPayload> FolderPayload = ContentBrowserAssetData::GetAssetFolderItemPayload(DataSource, InternalItem))
				{
					const FName InternalPath = FolderPayload->GetInternalPath();
					SelectedFolders.Add(InternalPath);
					Filter.PackagePaths.Add(InternalPath);
				}
				else if (TSharedPtr<const FContentBrowserAssetFileItemDataPayload> AssetPayload = ContentBrowserAssetData::GetAssetFileItemPayload(DataSource, InternalItem))
				{
					SelectedAssets.Add(AssetPayload->GetAssetData());
				}
			}
		}

		// Query for a list of assets in the selected paths
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
		AssetRegistryModule.Get().GetAssets(Filter, SelectedAssets);

		bContainsFolders = true;
	}

	if (IsValid())
	{
		AddSourceControlMenuOptions(InSection);
	}
}

bool FAssetSourceControlContextMenuState::IsValid() const
{
	return SelectedAssets.Num() > 0;
}

void FAssetSourceControlContextMenuState::Close()
{
	AsyncState.store(EAsyncState::Cancelled);

	if (SCCOperation.IsValid())
	{
		ISourceControlModule::Get().GetProvider().CancelOperation(SCCOperation.ToSharedRef());
	}
}

bool FAssetSourceControlContextMenuState::AddSourceControlMenuOptions(FToolMenuSection& InSection)
{
	if (ISourceControlModule::Get().IsEnabled())
	{
		if (ISourceControlModule::Get().GetProvider().IsAvailable())
		{
			// Start work to cache execution state, this can take a while if SCC cache is cold
			NextAsyncState();

			// SCC sub menu
			InSection.AddSubMenu(
				"SourceControlSubMenu",
				LOCTEXT("SourceControlSubMenuLabel", "Revision Control"),
				LOCTEXT("SourceControlSubMenuToolTip", "Revision Control actions."),
				FNewToolMenuDelegate::CreateSP(this, &FAssetSourceControlContextMenuState::FillSourceControlSubMenu),
				false,
				FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Icon", FRevisionControlStyleManager::GetStyleSetName() , "RevisionControl.Icon.ConnectedBadge")
			);
		}
		else
		{
			InSection.AddSubMenu(
				"SourceControlSubMenu",
				LOCTEXT("SourceControlSubMenuUnavailableLabel", "Revision Control (Unavailable)"),
				LOCTEXT("SourceControlSubMenuUnavailableToolTip", "Revision Control servers are currently unavailable, check your network connection."),
				FNewToolMenuDelegate(),
				FUIAction(
					FExecuteAction(),
					FCanExecuteAction::CreateLambda([]() { return false; })
				),
				EUserInterfaceActionType::Button,
				false,
				FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Icon", FRevisionControlStyleManager::GetStyleSetName() , "RevisionControl.Icon.WarningBadge")
			);
		}
	}
	else
	{
		InSection.AddMenuEntry(
			"SCCConnectToSourceControl",
			LOCTEXT("SCCConnectToSourceControl", "Connect to Revision Control..."),
			LOCTEXT("SCCConnectToSourceControlTooltip", "Connect to a revision control system for tracking changes to your content and levels."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Icon"),
			FUIAction(
				FExecuteAction::CreateLambda([]() { ISourceControlModule::Get().ShowLoginDialog(FSourceControlLoginClosed(), ELoginWindowMode::Modeless); }),
				FCanExecuteAction()
			)
		);
	}

	// Diff selected
	if (CanExecuteDiffSelected())
	{
		InSection.AddMenuEntry(
			"DiffSelected",
			LOCTEXT("DiffSelected", "Diff Selected"),
			LOCTEXT("DiffSelectedTooltip", "Diff the two assets that you have selected."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Diff"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAssetSourceControlContextMenuState::ExecuteDiffSelected)
			)
		);
	}
	else if (SelectedAssets.Num() == 1 && SelectedAssets[0].GetClass() && !bContainsFolders)
	{
		InSection.AddSubMenu(
			"DiffAgainst",
			LOCTEXT("DiffAgainst", "Diff Against..."),
			LOCTEXT("DiffAgainstTooltip", "Pick another asset of the same type to compare."),
			FNewToolMenuDelegate::CreateSP(this, &FAssetSourceControlContextMenuState::FillDiffAgainstSubMenu),
			false,
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Diff")
		);
	}

	return true;
}

void FAssetSourceControlContextMenuState::FillSourceControlSubMenu(UToolMenu* Menu)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	FToolMenuSection& Section = Menu->AddSection("AssetSourceControlActions", LOCTEXT("AssetSourceControlActionsMenuHeading", "Revision Control"));

	using namespace UE::AssetSourceControlContextMenu::Private;

	const bool bUsesCheckout = SourceControlProvider.UsesCheckout();
	const bool bUsesFileRevisions = SourceControlProvider.UsesFileRevisions();
	const bool bUsesSnapshots = SourceControlProvider.UsesSnapshots();
	const bool bUsesReadOnly = SourceControlProvider.UsesLocalReadOnlyState();
	const bool bUsesDiffAgainstDepot = SourceControlProvider.AllowsDiffAgainstDepot();
	const bool bUsesRemoteFilenames = SourceControlProvider.CanExecuteOperation(ISourceControlOperation::Create<FWhere>());

	if (bUsesFileRevisions)
	{
		AddAsyncMenuEntry(Section,
			"SCCSync",
			LOCTEXT("SCCSync", "Sync"),
			LOCTEXT("SCCSyncTooltip", "Updates the selected assets to the latest version in revision control."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Sync"),
			FUIAction(
				ExecutionCheck(FExecuteAction::CreateSP(this, &FAssetSourceControlContextMenuState::ExecuteSCCSync)),
				FCanExecuteAction::CreateSPLambda(this, [this]() { return IsActionEnabled(CanExecuteSCCSync()); })
			),
			FIsAsyncProcessingActive::CreateSPLambda(this, [this]() { return IsStillScanning(CanExecuteSCCSync()); })
		);
	}

	if (bUsesCheckout)
	{
		AddAsyncMenuEntry(Section,
			"SCCCheckOut",
			LOCTEXT("SCCCheckOut", "Check Out"),
			TAttribute<FText>::CreateLambda([pWeakThis = this->AsWeak()]()
				{
					TSharedPtr<FAssetSourceControlContextMenuState> pThis = pWeakThis.Pin();
					if (!pThis) return FText();

					if (pThis->IsStillScanning(pThis->CanExecuteSCCCheckOut()) || pThis->CheckedOutUsers.Num() == 0)
					{
						return LOCTEXT("SCCCheckOutTooltip", "Check out the selected assets from revision control.");
					}

					return FText::Format(LOCTEXT("SCCPartialCheckOut",
						"Checks out the selected assets from revision control that are not currently locked.\n\nLocked Assets:\n{0}"),
						pThis->CheckedOutUsersText);
				}),

			TAttribute<FSlateIcon>::CreateLambda([pWeakThis = this->AsWeak()]()
				{
					TSharedPtr<FAssetSourceControlContextMenuState> pThis = pWeakThis.Pin();
					if (!pThis) return FSlateIcon();

					return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(),
						pThis->CheckedOutUsers.Num() ? "RevisionControl.Actions.CheckOut" : "RevisionControl.Locked");
				}),
					FUIAction(
						ExecutionCheck(FExecuteAction::CreateSP(this, &FAssetSourceControlContextMenuState::ExecuteSCCCheckOut)),
						FCanExecuteAction::CreateSPLambda(this, [this]() { return IsActionEnabled(CanExecuteSCCCheckOut()); })
					),
					FIsAsyncProcessingActive::CreateSPLambda(this, [this]() { return IsStillScanning(CanExecuteSCCCheckOut()); })
					);

		AddAsyncMenuEntry(Section,
			"SCCSyncAndCheckOut",
			LOCTEXT("SCCSyncAndCheckOut", "Sync And Check Out"),
			TAttribute<FText>::CreateLambda([pWeakThis = this->AsWeak()]()
				{
					TSharedPtr<FAssetSourceControlContextMenuState> pThis = pWeakThis.Pin();
					if (!pThis) return FText();

					if (pThis->IsStillScanning(pThis->CanExecuteSCCCheckOut()) || pThis->CheckedOutUsers.Num() == 0)
					{
						return LOCTEXT("SCCSyncAndCheckOutTooltip", "Sync to latest and Check out the selected assets from revision control.");
					}

					return FText::Format(LOCTEXT("SCCPartialSyncAndCheckOut",
						"Sync to latest and Checks out the selected assets from revision control that are not currently locked.\n\nLocked Assets:\n{0}"),
						pThis->CheckedOutUsersText);
				}),

			TAttribute<FSlateIcon>::CreateLambda([pWeakThis = this->AsWeak()]()
				{
					TSharedPtr<FAssetSourceControlContextMenuState> pThis = pWeakThis.Pin();
					if (!pThis) return FSlateIcon();

					return FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(),
						pThis->CheckedOutUsers.Num() == 0 ? "RevisionControl.Actions.SyncAndCheckOut" : "RevisionControl.Locked");
				}),
					FUIAction(
						ExecutionCheck(FExecuteAction::CreateSP(this, &FAssetSourceControlContextMenuState::ExecuteSCCSyncAndCheckOut)),
						FCanExecuteAction::CreateSPLambda(this, [this]() { return IsActionEnabled(CanExecuteSCCSyncAndCheckOut()); })
					),
					FIsAsyncProcessingActive::CreateSPLambda(this, [this]() { return IsStillScanning(CanExecuteSCCSyncAndCheckOut()); })
					);
	}

	if (bUsesReadOnly)
	{
		AddAsyncMenuEntry(Section,
			"SCCMakeWritable",
			LOCTEXT("SCCMakeWritable", "Make Writable"),
			LOCTEXT("SCCMakeWritableTooltip", "Remove read-only flag from all selected assets."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.MakeWritable"),
			FUIAction(
				ExecutionCheck(FExecuteAction::CreateSP(this, &FAssetSourceControlContextMenuState::ExecuteSCCMakeWritable)),
				FCanExecuteAction::CreateSPLambda(this, [this]() { return IsActionEnabled(CanExecuteSCCMakeWritable()); })
			),
			FIsAsyncProcessingActive::CreateSPLambda(this, [this]() { return IsStillScanning(CanExecuteSCCMakeWritable()); })
		);
	}

	AddAsyncMenuEntry(Section,
		"SCCOpenForAdd",
		LOCTEXT("SCCOpenForAdd", "Mark For Add"),
		LOCTEXT("SCCOpenForAddTooltip", "Adds the selected assets to revision control."),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Add"),
		FUIAction(
			ExecutionCheck(FExecuteAction::CreateSP(this, &FAssetSourceControlContextMenuState::ExecuteSCCOpenForAdd)),
			FCanExecuteAction::CreateSPLambda(this, [this]() { return IsActionEnabled(CanExecuteSCCOpenForAdd()); })
		),
		FIsAsyncProcessingActive::CreateSPLambda(this, [this]() { return IsStillScanning(CanExecuteSCCOpenForAdd()); })
	);

	if (!bUsesSnapshots)
	{
		AddAsyncMenuEntry(Section,
			"SCCCheckIn",
			LOCTEXT("SCCCheckIn", "Check In"),
			LOCTEXT("SCCCheckInTooltip", "Checks the selected assets into revision control."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Submit"),
			FUIAction(
				ExecutionCheck(FExecuteAction::CreateSP(this, &FAssetSourceControlContextMenuState::ExecuteSCCCheckIn)),
				FCanExecuteAction::CreateSPLambda(this, [this]() { return IsActionEnabled(CanExecuteSCCCheckIn()); })
			),
			FIsAsyncProcessingActive::CreateSPLambda(this, [this]() { return IsStillScanning(CanExecuteSCCCheckIn()); })
		);
	}

	AddAsyncMenuEntry(Section,
		"SCCHistory",
		LOCTEXT("SCCHistory", "History"),
		LOCTEXT("SCCHistoryTooltip", "Displays the history of the selected asset in revision control."),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.History"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FAssetSourceControlContextMenuState::ExecuteSCCHistory),
			FCanExecuteAction::CreateSPLambda(this, [this]() { return IsActionEnabled(CanExecuteSCCHistory()); })
		),
		FIsAsyncProcessingActive::CreateSPLambda(this, [this]() { return IsStillScanning(CanExecuteSCCHistory()); })
	);

	if (bUsesDiffAgainstDepot)
	{
		AddAsyncMenuEntry(Section,
			"SCCDiffAgainstDepot",
			LOCTEXT("SCCDiffAgainstDepot", "Diff Against Depot"),
			LOCTEXT("SCCDiffAgainstDepotTooltip", "Look at differences between the local and remote version of the selected assets."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Diff"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAssetSourceControlContextMenuState::ExecuteSCCDiffAgainstDepot),
				FCanExecuteAction::CreateSPLambda(this, [this]() { return IsActionEnabled(CanExecuteSCCDiffAgainstDepot()); })
			),
			FIsAsyncProcessingActive::CreateSPLambda(this, [this]() { return IsStillScanning(CanExecuteSCCDiffAgainstDepot()); })
		);
	}

	AddAsyncMenuEntry(Section,
		"SCCRevert",
		LOCTEXT("SCCRevert", "Revert"),
		LOCTEXT("SCCRevertTooltip", "Reverts the selected assets to their original state from revision control."),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Revert"),
		FUIAction(
			FExecuteAction::CreateSP(this, &FAssetSourceControlContextMenuState::ExecuteSCCRevert),
			FCanExecuteAction::CreateSPLambda(this, [this]() { return IsActionEnabled(CanExecuteSCCRevert()); })
		),
		FIsAsyncProcessingActive::CreateSPLambda(this, [this]() { return IsStillScanning(CanExecuteSCCRevert()); })
	);

	if (bUsesReadOnly)
	{
		AddAsyncMenuEntry(Section,
			"SCCRevertWritable",
			LOCTEXT("SCCRevertWritable", "Revert Writable Files"),
			LOCTEXT("SCCRevertWritableTooltip", "Reverts the assets that are Writable to their current state from revision control. They will remain at their current revision."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Revert"),
			FUIAction(
				ExecutionCheck(FExecuteAction::CreateSP(this, &FAssetSourceControlContextMenuState::ExecuteSCCRevertWritable)),
				FCanExecuteAction::CreateSPLambda(this, [this]() { return IsActionEnabled(CanExecuteSCCRevertWritable()); })
			),
			FIsAsyncProcessingActive::CreateSPLambda(this, [this]() { return IsStillScanning(CanExecuteSCCRevertWritable()); })
		);
	}

	// At the end since this is the only item that will appear/disappear based on the content being looked at
	if (!bContainsFolders)
	{
		AddAsyncMenuEntry(Section,
			"SCCMerge",
			LOCTEXT("SCCMerge", "Merge"),
			LOCTEXT("SCCMergeTooltip", "Opens the blueprint editor with the merge tool open."),
			FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Merge"),
			FUIAction(
				FExecuteAction::CreateSP(this, &FAssetSourceControlContextMenuState::ExecuteSCCMerge),
				FCanExecuteAction::CreateSPLambda(this, [this]() { return IsActionEnabled(CanExecuteSCCMerge()); })),
			FIsAsyncProcessingActive::CreateSPLambda(this, [this]() { return IsStillScanning(CanExecuteSCCMerge()); })
		);
	}

	Section.AddMenuEntry(
		"SCCRefresh",
		LOCTEXT("SCCRefresh", "Refresh"),
		LOCTEXT("SCCRefreshTooltip", "Updates the revision control status of the asset."),
		FSlateIcon(FRevisionControlStyleManager::GetStyleSetName(), "RevisionControl.Actions.Refresh"),
		FUIAction(
			ExecutionCheck(FExecuteAction::CreateSP(this, &FAssetSourceControlContextMenuState::ExecuteSCCRefresh)),
			FCanExecuteAction()
		)
	);

	if (bUsesRemoteFilenames)
	{
		// Where check in p4v is a local conversion so very very fast, bump the number to reduce checks
		const int32 TriggerCheckAtNumActions = 100;

		Section.AddMenuEntry(
			"SCCCopyPaths",
			LOCTEXT("SCCCopyPaths", "Copy Remote Path(s)"),
			LOCTEXT("SCCCopyPathsTooltip", "Copy the revision control paths of the selected objects to the clipboard."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "GenericCommands.Copy"),
			FUIAction(
				ExecutionCheckSelectionOnly(FExecuteAction::CreateSP(this, &FAssetSourceControlContextMenuState::ExecuteSCCCopyPaths), TriggerCheckAtNumActions),
				FCanExecuteAction::CreateLambda([this]() { return IsActionEnabled(CanExecuteSCCCopyPaths()); })
			)
		);
	}
}

void FAssetSourceControlContextMenuState::FillDiffAgainstSubMenu(UToolMenu* Menu)
{
	if (SelectedAssets.Num() != 1 || SelectedAssets[0].GetClass() == nullptr)
	{
		return;
	}

	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.SelectionMode = ESelectionMode::Single;
	AssetPickerConfig.InitialThumbnailSize = EThumbnailSize::Tiny;
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.bAllowDragging = false;
	AssetPickerConfig.Filter.ClassPaths.Add(SelectedAssets[0].GetClass()->GetClassPathName());
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateLambda([this](const FAssetData& PickedAsset)
		{
			if (SelectedAssets.Num() == 1)
			{
				UObject* FirstObjectSelected = SelectedAssets[0].GetAsset();
				UObject* SecondObjectSelected = PickedAsset.GetAsset();

				if ((FirstObjectSelected != NULL) && (SecondObjectSelected != NULL))
				{
					if (FirstObjectSelected == SecondObjectSelected)
					{
						FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("SourceControlDiffSameAssets", "Can't diff an asset against itself."));
					}
					else
					{
						// Load the asset registry module
						FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

						FRevisionInfo CurrentRevision;
						CurrentRevision.Revision = TEXT("");

						// Need to dismiss context menu first or the diff window will try to attach to it
						FSlateApplication::Get().DismissAllMenus();

						AssetToolsModule.Get().DiffAssets(FirstObjectSelected, SecondObjectSelected, CurrentRevision, CurrentRevision);
					}
				}
			}
		});

	TSharedRef<SWidget> AssetPicker = 
		SNew(SBox)
	.WidthOverride(300.f)
	.HeightOverride(300.f)
	[
		FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser").Get().CreateAssetPicker(AssetPickerConfig)
	];

	FToolMenuSection& Section = Menu->AddSection("AssetDiffAgainstPicker", LOCTEXT("AssetDiffAgainstPickerHeading", "Diff Against..."));

	Section.AddEntry(FToolMenuEntry::InitWidget("DiffAgainst", AssetPicker, {}));
}

FExecuteAction FAssetSourceControlContextMenuState::ExecutionCheck(FExecuteAction&& InAction, int32 TriggerCheckAtNumActions) const
{
	return FExecuteAction::CreateSPLambda(this, [this, TriggerCheckAtNumActions, Action = MoveTemp(InAction)]()
		{
			if (SelectedAssets.Num() > TriggerCheckAtNumActions)
			{
				const FText Message = FText::Format(LOCTEXT("SCCLargeOperationWarningMessage", "You are about to perform this operation on a large amount of files ({0}), are you sure you want to continue?\n\nUnreal Editor may become unresponsive."), SelectedAssets.Num());
				const EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, Message, UE::AssetSourceControlContextMenu::Private::LargeOperationTitle);

				if (Result != EAppReturnType::Yes)
				{
					return;
				}
			}

			FScopedSlowTask SlowTask(0, LOCTEXT("SCCOperationSlowTaskLabel", "Performing Revision Control Operation"));
			SlowTask.MakeDialogDelayed(0.25f);
			Action.Execute();
		});
}

FExecuteAction FAssetSourceControlContextMenuState::ExecutionCheckSelectionOnly(FExecuteAction&& InAction, int32 TriggerCheckAtNumActions) const
{
	return FExecuteAction::CreateSPLambda(this, [this, TriggerCheckAtNumActions, Action = MoveTemp(InAction)]()
		{
			const int32 SelectionCount = SelectedFileCount + SelectedFolders.Num();
			if (SelectionCount > TriggerCheckAtNumActions)
			{
				const FText Message = FText::Format(LOCTEXT("SCCLargeOperationWarningMessageSelectionOnly", "You are about to perform this operation on a large amount of files/folders ({0}), are you sure you want to continue?\n\nUnreal Editor may become unresponsive."), SelectionCount);
				const EAppReturnType::Type Result = FMessageDialog::Open(EAppMsgType::YesNo, Message, UE::AssetSourceControlContextMenu::Private::LargeOperationTitle);

				if (Result != EAppReturnType::Yes)
				{
					return;
				}
			}

			FScopedSlowTask SlowTask(0, LOCTEXT("SCCOperationSlowTaskLabel", "Performing Revision Control Operation"));
			SlowTask.MakeDialogDelayed(0.25f);
			Action.Execute();
		});
}

void FAssetSourceControlContextMenuState::ExecuteDiffSelected() const
{
	if (SelectedAssets.Num() == 2)
	{
		UObject* FirstObjectSelected = SelectedAssets[0].GetAsset();
		UObject* SecondObjectSelected = SelectedAssets[1].GetAsset();

		if ((FirstObjectSelected != NULL) && (SecondObjectSelected != NULL))
		{
			// Load the asset registry module
			FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

			FRevisionInfo CurrentRevision;
			CurrentRevision.Revision = TEXT("");

			AssetToolsModule.Get().DiffAssets(FirstObjectSelected, SecondObjectSelected, CurrentRevision, CurrentRevision);
		}
	}
}

void FAssetSourceControlContextMenuState::ExecuteSCCMerge() const
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

	for (const FAssetData& AssetData : SelectedAssets)
	{
		if (UObject* CurrentObject = AssetData.GetAsset())
		{
			const FString PackagePath = AssetData.PackageName.ToString();
			const FString PackageName = AssetData.AssetName.ToString();
			auto AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(CurrentObject->GetClass()).Pin();
			if (AssetTypeActions.IsValid())
			{
				AssetTypeActions->Merge(CurrentObject);
			}
		}
	}
}

void FAssetSourceControlContextMenuState::ExecuteSCCCheckOut() const
{
	TArray<FString> PackageNames;
	GetSelectedPackageNames(PackageNames);
	FEditorFileUtils::CheckoutPackages(PackageNames);
}

void FAssetSourceControlContextMenuState::ExecuteSCCSyncAndCheckOut() const
{
	ExecuteSCCSync();
	ExecuteSCCCheckOut();
}

void FAssetSourceControlContextMenuState::ExecuteSCCMakeWritable() const
{
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
	FUncontrolledChangelistsModule& UncontrolledChangelistsModule = FUncontrolledChangelistsModule::Get();

	TArray<FString> PackageNames;
	GetSelectedPackageNames(PackageNames);

	for (const FString& Package : PackageNames)
	{
		PlatformFile.SetReadOnly(*SourceControlHelpers::PackageFilename(Package), false);
		UncontrolledChangelistsModule.OnMakeWritable(Package);
	}
}

void FAssetSourceControlContextMenuState::ExecuteSCCOpenForAdd() const
{
	TArray<FString> PackageNames;
	GetSelectedPackageNames(PackageNames);

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	TArray<FString> PackagesToAdd;
	TArray<UPackage*> PackagesToSave;
	for (const FString& PackageName : PackageNames)
	{
		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(SourceControlHelpers::PackageFilename(PackageName), EStateCacheUsage::Use);
		if (SourceControlState.IsValid() && !SourceControlState->IsSourceControlled())
		{
			PackagesToAdd.Add(PackageName);

			// Make sure the file actually exists on disk before adding it
			FString Filename;
			if (!FPackageName::DoesPackageExist(PackageName, &Filename))
			{
				if (UPackage* Package = FindPackage(NULL, *PackageName))
				{
					PackagesToSave.Add(Package);
				}
			}
		}
	}

	if (PackagesToAdd.Num() > 0)
	{
		// If any of the packages are new, save them now
		if (PackagesToSave.Num() > 0)
		{
			const bool bCheckDirty = false;
			const bool bPromptToSave = false;
			TArray<UPackage*> FailedPackages;
			const FEditorFileUtils::EPromptReturnCode Return = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave, &FailedPackages);
			if (FailedPackages.Num() > 0)
			{
				// don't try and add files that failed to save - remove them from the list
				for (UPackage* FailedPackage : FailedPackages)
				{
					PackagesToAdd.Remove(FailedPackage->GetName());
				}
			}
		}

		SourceControlProvider.Execute(ISourceControlOperation::Create<FMarkForAdd>(), SourceControlHelpers::PackageFilenames(PackagesToAdd));
	}
}

void FAssetSourceControlContextMenuState::ExecuteSCCCheckIn() const
{
	TArray<UPackage*> Packages;
	GetSelectedPackages(Packages);

	// Prompt the user to ask if they would like to first save any dirty packages they are trying to check-in
	const FEditorFileUtils::EPromptReturnCode UserResponse = FEditorFileUtils::PromptForCheckoutAndSave(Packages, true, true);

	// If the user elected to save dirty packages, but one or more of the packages failed to save properly OR if the user
	// canceled out of the prompt, don't follow through on the check-in process
	const bool bShouldProceed = (UserResponse == FEditorFileUtils::EPromptReturnCode::PR_Success || UserResponse == FEditorFileUtils::EPromptReturnCode::PR_Declined);
	if (bShouldProceed)
	{
		TArray<FString> PackageNames;
		GetSelectedPackageNames(PackageNames);

		const bool bUseSourceControlStateCache = true;

		FCheckinResultInfo ResultInfo;
		FSourceControlWindows::PromptForCheckin(ResultInfo, PackageNames, TArray<FString>(), TArray<FString>(), bUseSourceControlStateCache);

		if (ResultInfo.Result == ECommandResult::Failed)
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "SCC_Checkin_Failed", "Check-in failed as a result of save failure."));
		}
	}
	else
	{
		// If a failure occurred, alert the user that the check-in was aborted. This warning shouldn't be necessary if the user cancelled
		// from the dialog, because they obviously intended to cancel the whole operation.
		if (UserResponse == FEditorFileUtils::EPromptReturnCode::PR_Failure)
		{
			FMessageDialog::Open(EAppMsgType::Ok, NSLOCTEXT("UnrealEd", "SCC_Checkin_Aborted", "Check-in aborted as a result of save failure."));
		}
	}
}

void FAssetSourceControlContextMenuState::ExecuteSCCHistory() const
{
	TArray<FString> PackageNames;
	GetSelectedPackageNames(PackageNames);
	FSourceControlWindows::DisplayRevisionHistory(SourceControlHelpers::PackageFilenames(PackageNames));
}

void FAssetSourceControlContextMenuState::ExecuteSCCDiffAgainstDepot() const
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

	for (const FAssetData& AssetData : SelectedAssets)
	{
		if (UObject* CurrentObject = AssetData.GetAsset())
		{
			const FString PackagePath = AssetData.PackageName.ToString();
			const FString PackageName = AssetData.AssetName.ToString();
			AssetToolsModule.Get().DiffAgainstDepot(CurrentObject, PackagePath, PackageName);
		}
	}
}

void FAssetSourceControlContextMenuState::ExecuteSCCRevert() const
{
	TArray<FString> PackageNames;
	GetSelectedPackageNames(PackageNames);
	bool bReloadWorld = false;
	for (const FString& PackageName : PackageNames)
	{
		if (UPackage* Package = FindPackage(nullptr, *PackageName))
		{
			if (Package->ContainsMap())
			{
				bReloadWorld = true;
				break;
			}
		}
	}
	FSourceControlWindows::PromptForRevert(PackageNames, bReloadWorld);
}

void FAssetSourceControlContextMenuState::ExecuteSCCRevertWritable() const
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	TArray<FString> PackageNames;
	GetSelectedPackageNames(PackageNames);

	TArray<FString> PackageFileNamesToRevert;

	// Only add packages that can actually be reverted
	for (const FString& PackageName : PackageNames)
	{
		FString PackageFileName = SourceControlHelpers::PackageFilename(PackageName);

		if (!PlatformFile.IsReadOnly(*PackageFileName))
		{
			FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(PackageFileName, EStateCacheUsage::Use);
			if (SourceControlState.IsValid() && SourceControlState->IsSourceControlled() && !SourceControlState->IsCheckedOut())
			{
				PackageFileNamesToRevert.Add(PackageFileName);
			}
		}
	}

	auto RevertOperation = [](const TArray<FString>& InFilenames) -> bool
	{
		ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

		// Get history for all files so we know the current revision to revert to
		{
			TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateOperation = ISourceControlOperation::Create<FUpdateStatus>();
			UpdateOperation->SetUpdateHistory(true);
			SourceControlProvider.Execute(UpdateOperation, InFilenames);
		}

		// For sync files one at a time since we need to specify specific revisions on them
		for (const FString& PackageFileName : InFilenames)
		{
			FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(PackageFileName, EStateCacheUsage::Use);
			TSharedPtr<class ISourceControlRevision, ESPMode::ThreadSafe> Revision = SourceControlState->GetCurrentRevision();

			if (Revision.IsValid())
			{
				TSharedRef<FSync, ESPMode::ThreadSafe> SyncOperation = ISourceControlOperation::Create<FSync>();
				SyncOperation->SetRevision(Revision->GetRevision());
				SyncOperation->SetForce(true);
				if (SourceControlProvider.Execute(SyncOperation, PackageFileName) == ECommandResult::Succeeded)
				{
					PlatformFile.SetReadOnly(*PackageFileName, true);
				}
			}
		}

		return true;
	};

	SourceControlHelpers::ApplyOperationAndReloadPackages(PackageFileNamesToRevert, RevertOperation);

	// Tell UncontrolledCL to refresh to pick up changes to files it was watching.
	FUncontrolledChangelistsModule& UncontrolledChangelistsModule = FUncontrolledChangelistsModule::Get();
	UncontrolledChangelistsModule.UpdateStatus();
}

void FAssetSourceControlContextMenuState::ExecuteSCCSync() const
{
	TArray<FString> PackageNames;
	GetSelectedPackageNames(PackageNames);
	AssetViewUtils::SyncPackagesFromSourceControl(PackageNames);
}

bool FAssetSourceControlContextMenuState::CanExecuteSCCMerge() const
{
	FAssetToolsModule& AssetToolsModule = FModuleManager::GetModuleChecked<FAssetToolsModule>("AssetTools");

	if (bCanExecuteSCCMerge && SelectedAssets.Num() > 0)
	{
		for (const FAssetData& AssetData : SelectedAssets)
		{
			if (const UAssetDefinition* AssetDefinition = UAssetDefinitionRegistry::Get()->GetAssetDefinitionForAsset(AssetData))
			{
				if (!AssetDefinition->CanMerge())
				{
					return false;
				}
			}
			else
			{
				return false;
			}
		}

		return true;
	}

	return false;
}

void FAssetSourceControlContextMenuState::ExecuteSCCRefresh() const
{
	TArray<FString> PackageNames;
	GetSelectedPackageNames(PackageNames);

	TSharedPtr<FUpdateStatus, ESPMode::ThreadSafe> Operation = ISourceControlOperation::Create<FUpdateStatus>();
	Operation->SetForceUpdate(true);

	ISourceControlModule::Get().GetProvider().Execute(Operation.ToSharedRef(), SourceControlHelpers::PackageFilenames(PackageNames), EConcurrency::Asynchronous);
}

void FAssetSourceControlContextMenuState::ExecuteSCCCopyPaths() const
{
	TArray<FString> PackageNames;
	GetSelectedPackageNames(PackageNames);

	TArray<FString> PathsToConvert;
	for (const FName& Folder : SelectedFolders)
	{
		PathsToConvert.Add(FPaths::ConvertRelativePathToFull(FPackageName::LongPackageNameToFilename(Folder.ToString())));
	}

	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();

	for (const FString& PackageName : PackageNames)
	{
		FSourceControlStatePtr SourceControlState = SourceControlProvider.GetState(SourceControlHelpers::PackageFilename(PackageName), EStateCacheUsage::Use);
		if (SourceControlState.IsValid() && SourceControlState->IsSourceControlled())
		{
			// Don't include any files that are under the folders in the selection
			if (!SelectedFolders.ContainsByPredicate([&PackageName](const FName& Folder) { return PackageName.StartsWith(Folder.ToString()); }))
			{
				PathsToConvert.Add(FPaths::ConvertRelativePathToFull(SourceControlState->GetFilename()));
			}
		}
	}

	TSharedRef<FWhere> Operation = ISourceControlOperation::Create<FWhere>();

	ISourceControlModule::Get().GetProvider().Execute(Operation, PathsToConvert, EConcurrency::Synchronous,
		FSourceControlOperationComplete::CreateLambda([Operation, NumFolders = SelectedFolders.Num(), &SourceControlProvider](const FSourceControlOperationRef& InOperation, ECommandResult::Type Result)
			{
				if (Result == ECommandResult::Succeeded)
				{
					TArray<const FString*> RemotePaths;
					RemotePaths.Reserve(Operation->GetFiles().Num());
					Algo::Transform(Operation->GetFiles(), RemotePaths, [](const FWhere::FileInfo& Path) { return &Path.RemotePath; });

					TArrayView<const FString*> RemotePathsView = RemotePaths;

					// Sort folders and files separately to keep folders first
					Algo::Sort(RemotePathsView.Left(NumFolders), [](const FString* Left, const FString* Right) { return Left->Compare(*Right) < 0; });
					Algo::Sort(RemotePathsView.Right(RemotePaths.Num() - NumFolders), [](const FString* Left, const FString* Right) { return Left->Compare(*Right) < 0; });

					FString ClipboardText;

					for (const FString* Path : RemotePaths)
					{
						if (ClipboardText.Len() > 0)
						{
							ClipboardText += LINE_TERMINATOR;
						}

						ClipboardText += *Path;
					}

					FPlatformApplicationMisc::ClipboardCopy(*ClipboardText);
				}
				else
				{
					// Todo: Error?
				}
			})
	);
}

bool FAssetSourceControlContextMenuState::CanExecuteSCCCheckOut() const
{
	return bCanExecuteSCCCheckOut;
}

bool FAssetSourceControlContextMenuState::CanExecuteSCCSyncAndCheckOut() const
{
	return bCanExecuteSCCSyncAndCheckOut;
}

bool FAssetSourceControlContextMenuState::CanExecuteSCCMakeWritable() const
{
	return bCanExecuteSCCMakeWritable;
}

bool FAssetSourceControlContextMenuState::CanExecuteSCCOpenForAdd() const
{
	return bCanExecuteSCCOpenForAdd;
}

bool FAssetSourceControlContextMenuState::CanExecuteSCCCheckIn() const
{
	return bCanExecuteSCCCheckIn;
}

bool FAssetSourceControlContextMenuState::CanExecuteSCCHistory() const
{
	return bCanExecuteSCCHistory;
}

bool FAssetSourceControlContextMenuState::CanExecuteSCCDiffAgainstDepot() const
{
	return bCanExecuteSCCHistory;
}

bool FAssetSourceControlContextMenuState::CanExecuteSCCRevert() const
{
	return bCanExecuteSCCRevert;
}

bool FAssetSourceControlContextMenuState::CanExecuteSCCRevertWritable() const
{
	return bCanExecuteSCCRevertWritable;
}

bool FAssetSourceControlContextMenuState::CanExecuteSCCSync() const
{
	return bCanExecuteSCCSync;
}

bool FAssetSourceControlContextMenuState::CanExecuteSCCCopyPaths() const
{
	return bCanExecuteSCCCopyPaths;
}

bool FAssetSourceControlContextMenuState::CanExecuteDiffSelected() const
{
	bool bCanDiffSelected = false;
	if (SelectedAssets.Num() == 2 && !bContainsFolders)
	{
		const FAssetData& FirstSelection = SelectedAssets[0];
		const FAssetData& SecondSelection = SelectedAssets[1];

		bCanDiffSelected = FirstSelection.AssetClassPath == SecondSelection.AssetClassPath;
	}

	return bCanDiffSelected;
}

bool FAssetSourceControlContextMenuState::AllowExecuteSCCRevertUnsaved() const
{
	if (IConsoleVariable* CVar = IConsoleManager::Get().FindConsoleVariable(TEXT("SourceControl.RevertUnsaved.Enable")))
	{
		return CVar->GetBool();
	}
	else
	{
		return false;
	}
}

bool FAssetSourceControlContextMenuState::IsActionEnabled(bool bEnabled) const
{
	return bEnabled && !IsStillScanning(bEnabled);
}

bool FAssetSourceControlContextMenuState::IsStillScanning(bool bEnabled) const
{
	// If we only have files selected then we only care about AsyncState
	// If at least one folder is selected then we will consider done if at least
	// one file was found that supports the given operation.
	return AsyncState != EAsyncState::Idle && (bContainsFolders ? !bEnabled : true);
}

bool FAssetSourceControlContextMenuState::TrySetAsyncState(EAsyncState FromState, EAsyncState ToState)
{
	return AsyncState.compare_exchange_strong(FromState, ToState);
}

void FAssetSourceControlContextMenuState::NextAsyncState()
{
	switch (AsyncState)
	{
	case EAsyncState::None:
		if (TrySetAsyncState(EAsyncState::None, EAsyncState::TryCacheCanExecute))
		{
			Async(EAsyncExecution::TaskGraphMainThread, [this, This = AsShared()]()
				{
					TArray<FString> SelectedPaths;
					Algo::TransformIf(SelectedAssets, SelectedPaths,
						[](const FAssetData& AssetData) { return AssetData.IsValid(); },
						[](const FAssetData& AssetData) { return SourceControlHelpers::PackageFilename(AssetData.PackageName.ToString()); });

					TryCacheCanExecuteVars(SelectedPaths, &PathsWithUnknownState);

					NextAsyncState();
				});
		}
		break;
	case EAsyncState::TryCacheCanExecute:
		if (PathsWithUnknownState.Num() == 0)
		{
			// Skip to the end since there's no more work to do
			TrySetAsyncState(EAsyncState::TryCacheCanExecute, EAsyncState::Idle);
		}
		else if (TrySetAsyncState(EAsyncState::TryCacheCanExecute, EAsyncState::SCCUpdate))
		{
			// Do an async force update to make sure to get mode up-to-date status
			FSourceControlOperationRef Operation = ISourceControlOperation::Create<FUpdateStatus>();
			SCCOperation = Operation;

			ECommandResult::Type Result = ISourceControlModule::Get().GetProvider().Execute(Operation, PathsWithUnknownState, EConcurrency::Asynchronous,
				FSourceControlOperationComplete::CreateSP(this, &FAssetSourceControlContextMenuState::CacheCanExecuteVarsSCCHandler));

			if (Result != ECommandResult::Succeeded)
			{
				AsyncState = EAsyncState::Error;
			}
		}
		break;
	case EAsyncState::SCCUpdate:
		TrySetAsyncState(EAsyncState::SCCUpdate, EAsyncState::Idle);
		break;
	}
}

void FAssetSourceControlContextMenuState::CacheCanExecuteVarsSCCHandler(const FSourceControlOperationRef& InOperation, ECommandResult::Type Result)
{
	EAsyncState ExpectedState = EAsyncState::SCCUpdate;
	if (Result == ECommandResult::Succeeded)
	{
		TryCacheCanExecuteVars(PathsWithUnknownState, nullptr);
	}

	NextAsyncState();
}

void FAssetSourceControlContextMenuState::TryCacheCanExecuteVars(const TArray<FString>& InSelectedPaths, TArray<FString>* OptionalPathsWithUnknownState)
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	const bool bUsesCheckout = SourceControlProvider.UsesCheckout();
	const bool bUsesFileRevisions = SourceControlProvider.UsesFileRevisions();
	const bool bUsesSnapshots = SourceControlProvider.UsesSnapshots();

	// If a package is dirty, allow a revert of the in-memory changes that have not yet been saved to disk.
	if (AllowExecuteSCCRevertUnsaved())
	{
		for (const FString& SelectedPath : InSelectedPaths)
		{
			FString PackageName;
			if (FPackageName::TryConvertFilenameToLongPackageName(SelectedPath, PackageName))
			{
				if (UPackage* Package = FindPackage(NULL, *PackageName))
				{
					if (Package->IsDirty())
					{
						bCanExecuteSCCRevert = true;
					}
				}
			}
		}
	}

	// Check the SCC state for each package in the selected paths - this needs to happen on the main thread
	TArray<FSourceControlStateRef> SelectedStates;
	if (SourceControlProvider.GetState(InSelectedPaths, SelectedStates, EStateCacheUsage::Use) == ECommandResult::Succeeded)
	{
		for (const FSourceControlStateRef& SourceControlState : SelectedStates)
		{
			if (AsyncState == EAsyncState::Cancelled)
				return;

			if (SourceControlState->IsUnknown())
			{
				if (OptionalPathsWithUnknownState != nullptr)
				{
					OptionalPathsWithUnknownState->Add(SourceControlState->GetFilename());
				}
				continue;
			}

			if (SourceControlState->IsConflicted())
			{
				bCanExecuteSCCMerge = true;
			}

			bool bCanSyncCurrentItem = false;
			const bool bIsReadOnly = PlatformFile.IsReadOnly(*SourceControlState->GetFilename());

			if (bUsesFileRevisions)
			{
				if (!SourceControlState->IsCurrent())
				{
					bCanExecuteSCCSync = bCanSyncCurrentItem = true;
				}
			}

			if (!bUsesSnapshots)
			{
				if (SourceControlState->CanCheckIn())
				{
					bCanExecuteSCCCheckIn = true;
				}				
			}

			if (bUsesCheckout)
			{
				if (SourceControlState->CanCheckout())
				{
					bCanExecuteSCCCheckOut = true;
				}
				else
				{
					FString CheckedOutOther;
					if (!bContainsFolders && SourceControlState->IsCheckedOutOther(&CheckedOutOther))
					{
						CheckedOutUsers.Add(FString::Printf(TEXT("%s checked out by %s"), *SourceControlState->GetFilename(), *CheckedOutOther));
					}
					else if (bCanSyncCurrentItem)
					{
						bCanExecuteSCCSyncAndCheckOut = true;
					}
				}

				if (bIsReadOnly)
				{
					bCanExecuteSCCMakeWritable = true;
				}

				if (SourceControlState->IsSourceControlled() && !SourceControlState->IsCheckedOut() && !bIsReadOnly)
				{
					bCanExecuteSCCRevertWritable = true;
				}
			}

			if (!SourceControlState->IsSourceControlled())
			{
				bCanExecuteSCCOpenForAdd = SourceControlState->CanAdd();
			}
			else
			{
				bCanExecuteSCCHistory = !SourceControlState->IsAdded();
				bCanExecuteSCCCopyPaths = true;
			}

			if (SourceControlState->CanRevert())
			{
				bCanExecuteSCCRevert = true;
			}
		}
	}

	CheckedOutUsersText = FText::FromString(FString::Join(CheckedOutUsers, TEXT("\n")));
}

void FAssetSourceControlContextMenuState::GetSelectedPackageNames(TArray<FString>& OutPackageNames) const
{
	for (const FAssetData& AssetData : SelectedAssets)
	{
		OutPackageNames.Add(AssetData.PackageName.ToString());
	}
}

void FAssetSourceControlContextMenuState::GetSelectedPackages(TArray<UPackage*>& OutPackages) const
{
	for (const FAssetData& AssetData : SelectedAssets)
	{
		if (UPackage* Package = FindPackage(NULL, *AssetData.PackageName.ToString()))
		{
			OutPackages.Add(Package);
		}
	}
}

#undef LOCTEXT_NAMESPACE
