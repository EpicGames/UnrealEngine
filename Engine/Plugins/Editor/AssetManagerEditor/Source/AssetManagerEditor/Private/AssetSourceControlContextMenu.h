// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

class UToolMenu;
struct FToolMenuSection;
class FAssetSourceControlContextMenuState;

// Display source control options in the given menu section based on what's selected in the menu context
class FAssetSourceControlContextMenu
{
public:
	~FAssetSourceControlContextMenu();

	/** Makes the context menu widget */
	void MakeContextMenu(FToolMenuSection& InSection);

private:
	TSharedPtr<FAssetSourceControlContextMenuState> InnerState;
};
