// Copyright Epic Games, Inc. All Rights Reserved.

#include "PrimaryAssetIdCustomization.h"
#include "AssetManagerEditorModule.h"
#include "EdGraph/EdGraphSchema.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Engine/AssetManager.h"
#include "PropertyCustomizationHelpers.h"
#include "Editor.h"
#include "AssetThumbnail.h"
#include "ScopedTransaction.h"
#include "Widgets/SOverlay.h"

#define LOCTEXT_NAMESPACE "PrimaryAssetIdCustomization"

void FPrimaryAssetIdCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	check(UAssetManager::IsInitialized());

	StructPropertyHandle = InStructPropertyHandle;

	const FString& TypeFilterString = StructPropertyHandle->GetMetaData("AllowedTypes");
	if( !TypeFilterString.IsEmpty() )
	{
		TArray<FString> CustomTypeFilterNames;
		TypeFilterString.ParseIntoArray(CustomTypeFilterNames, TEXT(","), true);

		for(auto It = CustomTypeFilterNames.CreateConstIterator(); It; ++It)
		{
			const FString& TypeName = *It;

			AllowedTypes.Add(*TypeName);
		}
	}

	bool bShowThumbnail = true;
	if (StructPropertyHandle->HasMetaData("DisplayThumbnail"))
	{
		const FString& ShowThumbnailString = StructPropertyHandle->GetMetaData("DisplayThumbnail");
		bShowThumbnail = ShowThumbnailString.Len() == 0 || ShowThumbnailString == TEXT("true");
	}

	TAttribute<bool> bAllowClear = MakeAttributeLambda([this]() -> bool
		{
			return !(StructPropertyHandle->GetMetaDataProperty()->PropertyFlags & CPF_NoClear);
		});

	AllowedClasses = PropertyCustomizationHelpers::GetClassesFromMetadataString(StructPropertyHandle->GetMetaData("AllowedClasses"));
	DisallowedClasses = PropertyCustomizationHelpers::GetClassesFromMetadataString(StructPropertyHandle->GetMetaData("DisallowedClasses"));

	TSharedRef OptionsBox = SNew(SHorizontalBox)
		// Use Selected
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			PropertyCustomizationHelpers::MakeUseSelectedButton(FSimpleDelegate::CreateSP(this, &FPrimaryAssetIdCustomization::OnUseSelected))
		]
		// Browse To
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			PropertyCustomizationHelpers::MakeBrowseButton(FSimpleDelegate::CreateSP(this, &FPrimaryAssetIdCustomization::OnBrowseTo))
		]
		// Clear
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			PropertyCustomizationHelpers::MakeClearButton(FSimpleDelegate::CreateSP(this, &FPrimaryAssetIdCustomization::OnClear), LOCTEXT("ClearButtonToolTipText", "Clear"), bAllowClear)
		];

	TSharedRef<SHorizontalBox> ValueBox = SNew(SHorizontalBox);

	if (bShowThumbnail)
	{
		const int32 ThumbnailSize = 64;
		AssetThumbnail = MakeShareable(new FAssetThumbnail(FAssetData(), ThumbnailSize, ThumbnailSize, StructCustomizationUtils.GetThumbnailPool()));
		UpdateThumbnail();

		// Thumbnail
		ValueBox->AddSlot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(0, 0, 4, 0)
			[
				SNew(SOverlay)
				+ SOverlay::Slot()
				.Padding(1)
				[
					SAssignNew(ThumbnailBorder, SBorder)
					.Padding(0)
					.BorderImage(FStyleDefaults::GetNoBrush())
					.OnMouseDoubleClick(this, &FPrimaryAssetIdCustomization::OnAssetThumbnailDoubleClick)
					[
						SNew(SBox)
						.WidthOverride(ThumbnailSize)
						.HeightOverride(ThumbnailSize)
						[
							AssetThumbnail->MakeThumbnailWidget()
						]
					]
				]
				+ SOverlay::Slot()
				[
					SNew(SImage)
					.Image(this, &FPrimaryAssetIdCustomization::GetThumbnailBorder)
					.Visibility(EVisibility::SelfHitTestInvisible)
				]
			];

		TAttribute<FText> AssetNameAttribute = MakeAttributeSP(this, &FPrimaryAssetIdCustomization::GetAssetNameText);

		// Use thumbnail vertical real-estate to show Asset Name on top of the ComboBox
		ValueBox->AddSlot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SNew(SVerticalBox)
					// Asset Name
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Top)
					.AutoHeight()
					.Padding(FMargin(6.f, 6.f, 0.0f, 4.0f))
					[
						SNew(STextBlock)
							.Text(AssetNameAttribute)
							.ToolTipText(AssetNameAttribute)
							.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
							.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.BoldFont")))
					]
					// PrimaryAssetIdSelector
					+ SVerticalBox::Slot()
					.VAlign(VAlign_Center)
					.FillHeight(1.f)
					[
						IAssetManagerEditorModule::MakePrimaryAssetIdSelector(
							FOnGetPrimaryAssetDisplayText::CreateSP(this, &FPrimaryAssetIdCustomization::GetDisplayText),
							FOnSetPrimaryAssetId::CreateSP(this, &FPrimaryAssetIdCustomization::OnIdSelected), bAllowClear.Get(), AllowedTypes, AllowedClasses,
							DisallowedClasses)
					]
					// Options
					+ SVerticalBox::Slot()
					.Padding(FMargin(4.f, 4.f))
					.AutoHeight()
					.VAlign(VAlign_Bottom)
					[
						OptionsBox
					]
			];
	}
	else // No thumbnail
	{
		// PrimaryAssetIdSelector
		ValueBox->AddSlot()
			.FillWidth(1.0f)
			.Padding(2.0f)
			.VAlign(VAlign_Center)
			[
					IAssetManagerEditorModule::MakePrimaryAssetIdSelector(
						{
							.OnGetDisplayText = FOnGetPrimaryAssetDisplayText::CreateSP(this, &FPrimaryAssetIdCustomization::GetFormattedAssetDisplayText),
							.OnGetTooltipText = FOnGetPrimaryAssetDisplayText::CreateSP(this, &FPrimaryAssetIdCustomization::GetDisplayText),
							.OnSetId = FOnSetPrimaryAssetId::CreateSP(this, &FPrimaryAssetIdCustomization::OnIdSelected),
							.bAllowClear = bAllowClear.Get(),
							.AllowedTypes = AllowedTypes,
							.AllowedClasses = AllowedClasses,
							.DisallowedClasses = DisallowedClasses
						})
			];

		// Options
		ValueBox->AddSlot()
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Right)
			.Padding(FMargin(4.0f, 0.0f))
			.AutoWidth()
			[
				OptionsBox
			];
	}


	HeaderRow
	.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.0f)
	.MaxDesiredWidth(0.0f)
	[
		ValueBox
	];
}

const FSlateBrush* FPrimaryAssetIdCustomization::GetThumbnailBorder() const
{
	static const FName HoveredBorderName("PropertyEditor.AssetThumbnailBorderHovered");
	static const FName RegularBorderName("PropertyEditor.AssetThumbnailBorder");

	return ThumbnailBorder->IsHovered() ? FAppStyle::Get().GetBrush(HoveredBorderName) : FAppStyle::Get().GetBrush(RegularBorderName);
}

void FPrimaryAssetIdCustomization::OnIdSelected(FPrimaryAssetId AssetId)
{
	if (StructPropertyHandle.IsValid() && StructPropertyHandle->IsValidHandle())
	{
		StructPropertyHandle->SetValueFromFormattedString(AssetId.ToString());
	}

	UpdateThumbnail();
}

FText FPrimaryAssetIdCustomization::GetAssetNameText() const
{
	FString StringReference;

	FPrimaryAssetId PrimaryAssetId = GetCurrentPrimaryAssetId();
	if (PrimaryAssetId.IsValid())
	{
		StructPropertyHandle->GetValueAsFormattedString(StringReference);

		FAssetData AssetData;
		if (UAssetManager::Get().GetPrimaryAssetData(PrimaryAssetId, AssetData))
		{
			StringReference = AssetData.AssetName.ToString();
		}
	}

	return FText::AsCultureInvariant(StringReference);

}

FText FPrimaryAssetIdCustomization::GetDisplayText() const
{
	FString StringReference;

	if (StructPropertyHandle.IsValid())
	{
		StructPropertyHandle->GetValueAsFormattedString(StringReference);
	}

	return !StringReference.IsEmpty() ? FText::AsCultureInvariant(StringReference) : LOCTEXT("EmptyPlaceholderText", "None");
}

FText FPrimaryAssetIdCustomization::GetFormattedAssetDisplayText() const
{
	FText DisplayText = GetDisplayText();

	return DisplayText.EqualTo(LOCTEXT("EmptyPlaceholderText", "None")) ? DisplayText :
		FText::Format(
			FText::AsCultureInvariant(TEXT("{0} ( {1} )")),
			GetAssetNameText(),
			DisplayText
		);
}

FPrimaryAssetId FPrimaryAssetIdCustomization::GetCurrentPrimaryAssetId() const
{
	FString StringReference;
	if (StructPropertyHandle.IsValid())
	{
		StructPropertyHandle->GetValueAsFormattedString(StringReference);
	}
	else
	{
		StringReference = FPrimaryAssetId().ToString();
	}

	return FPrimaryAssetId(StringReference);
}

void FPrimaryAssetIdCustomization::UpdateThumbnail()
{
	if (AssetThumbnail.IsValid())
	{
		FAssetData AssetData;
		FPrimaryAssetId PrimaryAssetId = GetCurrentPrimaryAssetId();
		if (PrimaryAssetId.IsValid())
		{
			UAssetManager::Get().GetPrimaryAssetData(PrimaryAssetId, AssetData);
		}

		AssetThumbnail->SetAsset(AssetData);
	}
}

void FPrimaryAssetIdCustomization::OnBrowseTo()
{
	FPrimaryAssetId PrimaryAssetId = GetCurrentPrimaryAssetId();

	if (PrimaryAssetId.IsValid())
	{
		FAssetData FoundData;
		if (UAssetManager::Get().GetPrimaryAssetData(PrimaryAssetId, FoundData))
		{
			TArray<FAssetData> SyncAssets;
			SyncAssets.Add(FoundData);
			GEditor->SyncBrowserToObjects(SyncAssets);
		}
	}	
}

void FPrimaryAssetIdCustomization::OnUseSelected()
{
	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);

	for (const FAssetData& AssetData : SelectedAssets)
	{
		FPrimaryAssetId PrimaryAssetId = UAssetManager::Get().GetPrimaryAssetIdForData(AssetData);
		if (PrimaryAssetId.IsValid())
		{
			OnIdSelected(PrimaryAssetId);
			return;
		}
	}
}

void FPrimaryAssetIdCustomization::OnOpenAssetEditor()
{
	FAssetData AssetData;
	FPrimaryAssetId PrimaryAssetId = GetCurrentPrimaryAssetId();
	if (PrimaryAssetId.IsValid())
	{
		if (UAssetManager::Get().GetPrimaryAssetData(PrimaryAssetId, AssetData))
		{
			GEditor->EditObject(AssetData.GetAsset());
		}
	}
}

void FPrimaryAssetIdCustomization::OnClear()
{
	OnIdSelected(FPrimaryAssetId());
}

FReply FPrimaryAssetIdCustomization::OnAssetThumbnailDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent)
{
	OnOpenAssetEditor();
	return FReply::Handled();
}

void SPrimaryAssetIdGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SPrimaryAssetIdGraphPin::GetDefaultValueWidget()
{
	FString DefaultString = GraphPinObj->GetDefaultAsString();
	CurrentId = FPrimaryAssetId(DefaultString);
	
	TArray<FPrimaryAssetType> AllowedTypes;
	if (UEdGraphNode* OwningNode = GraphPinObj->GetOwningNode())
	{
		const FString TypeFilterString = OwningNode->GetPinMetaData(GraphPinObj->GetFName(), TEXT("AllowedTypes"));
		if( !TypeFilterString.IsEmpty() )
		{
			TArray<FString> CustomTypeFilterNames;
			TypeFilterString.ParseIntoArray(CustomTypeFilterNames, TEXT(","), true);

			for(auto It = CustomTypeFilterNames.CreateConstIterator(); It; ++It)
			{
				const FString& TypeName = *It;
				AllowedTypes.Add(*TypeName);
			}
		}
	}

	return SNew(SHorizontalBox)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			IAssetManagerEditorModule::MakePrimaryAssetIdSelector(
				FOnGetPrimaryAssetDisplayText::CreateSP(this, &SPrimaryAssetIdGraphPin::GetDisplayText),
				FOnSetPrimaryAssetId::CreateSP(this, &SPrimaryAssetIdGraphPin::OnIdSelected),
				true, AllowedTypes)
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1, 0)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ButtonColorAndOpacity(this, &SPrimaryAssetIdGraphPin::OnGetWidgetBackground)
			.OnClicked(this, &SPrimaryAssetIdGraphPin::OnUseSelected)
			.ContentPadding(1.f)
			.ToolTipText(NSLOCTEXT("GraphEditor", "ObjectGraphPin_Use_Tooltip", "Use asset browser selection"))
			[
				SNew(SImage)
				.ColorAndOpacity(this, &SPrimaryAssetIdGraphPin::OnGetWidgetForeground)
				.Image(FAppStyle::GetBrush(TEXT("Icons.CircleArrowLeft")))
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.Padding(1, 0)
		.VAlign(VAlign_Center)
		[
			SNew(SButton)
			.ButtonStyle(FAppStyle::Get(), "NoBorder")
			.ButtonColorAndOpacity(this, &SPrimaryAssetIdGraphPin::OnGetWidgetBackground)
			.OnClicked(this, &SPrimaryAssetIdGraphPin::OnBrowseTo)
			.ContentPadding(0)
			.ToolTipText(NSLOCTEXT("GraphEditor", "ObjectGraphPin_Browse_Tooltip", "Browse"))
			[
				SNew(SImage)
				.ColorAndOpacity(this, &SPrimaryAssetIdGraphPin::OnGetWidgetForeground)
				.Image(FAppStyle::GetBrush(TEXT("Icons.Search")))
			]
		];
}

FSlateColor SPrimaryAssetIdGraphPin::OnGetWidgetForeground() const
{
	float Alpha = (IsHovered() || bOnlyShowDefaultValue) ? 1.f : 0.15f;
	return FSlateColor(FLinearColor(1.f, 1.f, 1.f, Alpha));
}

FSlateColor SPrimaryAssetIdGraphPin::OnGetWidgetBackground() const
{
	float Alpha = (IsHovered() || bOnlyShowDefaultValue) ? 0.8f : 0.4f;
	return FSlateColor(FLinearColor(1.f, 1.f, 1.f, Alpha));
}

void SPrimaryAssetIdGraphPin::OnIdSelected(FPrimaryAssetId AssetId)
{
	CurrentId = AssetId;

	if (CurrentId.ToString() != GraphPinObj->GetDefaultAsString())
	{
		const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangePinValue", "Change Pin Value"));
		GraphPinObj->Modify();
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, CurrentId.ToString());
	}
}

FText SPrimaryAssetIdGraphPin::GetDisplayText() const
{
	return FText::AsCultureInvariant(CurrentId.ToString());
}

FReply SPrimaryAssetIdGraphPin::OnBrowseTo()
{
	if (CurrentId.IsValid())
	{
		FAssetData FoundData;
		if (UAssetManager::Get().GetPrimaryAssetData(CurrentId, FoundData))
		{
			TArray<FAssetData> SyncAssets;
			SyncAssets.Add(FoundData);
			GEditor->SyncBrowserToObjects(SyncAssets);
		}
	}

	return FReply::Handled();
}

FReply SPrimaryAssetIdGraphPin::OnUseSelected()
{
	TArray<FAssetData> SelectedAssets;
	GEditor->GetContentBrowserSelections(SelectedAssets);

	for (const FAssetData& AssetData : SelectedAssets)
	{
		FPrimaryAssetId PrimaryAssetId = UAssetManager::Get().GetPrimaryAssetIdForData(AssetData);
		if (PrimaryAssetId.IsValid())
		{
			OnIdSelected(PrimaryAssetId);
			return FReply::Handled();
		}
	}
	return FReply::Handled();
}


#undef LOCTEXT_NAMESPACE
