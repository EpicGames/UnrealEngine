// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "SGraphPin.h"
#include "UObject/PrimaryAssetId.h"

class FAssetThumbnail;
class FDetailWidgetRow;
class IDetailChildrenBuilder;
class IPropertyHandle;
class SBorder;

/** Customization for a primary asset id, shows an asset picker with filters */
class FPrimaryAssetIdCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FPrimaryAssetIdCustomization);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}

private:
	void OnIdSelected(FPrimaryAssetId AssetId);
	void OnBrowseTo();
	void OnClear();
	void OnUseSelected();
	FText GetAssetNameText() const;
	FText GetDisplayText() const;
	FText GetFormattedAssetDisplayText() const;
	FPrimaryAssetId GetCurrentPrimaryAssetId() const;
	void UpdateThumbnail();
	void OnOpenAssetEditor();

	/** Gets the border brush to show around the thumbnail, changes when the user hovers on it. */
	const FSlateBrush* GetThumbnailBorder() const;

	/**
	 * Handle double clicking the asset thumbnail. this 'Edits' the displayed asset
	 */
	FReply OnAssetThumbnailDoubleClick(const FGeometry& InMyGeometry, const FPointerEvent& InMouseEvent);

	/** Handle to the struct property being customized */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** Specified type */
	TArray<FPrimaryAssetType> AllowedTypes;

	/** Classes which can be selected with this PrimaryAssetId  */
	TArray<const UClass*> AllowedClasses;

	/** Classes which cannot be selected with this PrimaryAssetId  */
	TArray<const UClass*> DisallowedClasses;
	
	/** Thumbnail resource */
	TSharedPtr<FAssetThumbnail> AssetThumbnail;

	/** The border surrounding the thumbnail image. */
	TSharedPtr<SBorder> ThumbnailBorder;
};

/** Graph pin version of UI */
class SPrimaryAssetIdGraphPin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SPrimaryAssetIdGraphPin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

private:

	void OnIdSelected(FPrimaryAssetId AssetId);
	FText GetDisplayText() const;
	FSlateColor OnGetWidgetForeground() const;
	FSlateColor OnGetWidgetBackground() const;
	FReply OnBrowseTo();
	FReply OnUseSelected();

	FPrimaryAssetId CurrentId;
};
