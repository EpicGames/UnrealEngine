// Copyright Epic Games, Inc. All Rights Reserved.

#include "PrimaryAssetTypeCustomization.h"
#include "AssetManagerEditorModule.h"
#include "EdGraph/EdGraphSchema.h"
#include "Widgets/Text/STextBlock.h"
#include "Engine/AssetManager.h"
#include "PropertyCustomizationHelpers.h"
#include "ScopedTransaction.h"

#define LOCTEXT_NAMESPACE "PrimaryAssetTypeCustomization"

void FPrimaryAssetTypeCustomization::CustomizeHeader(TSharedRef<class IPropertyHandle> InStructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	check(UAssetManager::IsInitialized());

	StructPropertyHandle = InStructPropertyHandle;
	const bool bAllowClear = !StructPropertyHandle->GetMetaDataProperty()->HasAnyPropertyFlags(CPF_NoClear);

	HeaderRow
	.NameContent()
	[
		InStructPropertyHandle->CreatePropertyNameWidget()
	]
	.ValueContent()
	.MinDesiredWidth(250.0f)
	.MaxDesiredWidth(0.0f)
	[
		PropertyCustomizationHelpers::MakePropertyComboBox(InStructPropertyHandle, FOnGetPropertyComboBoxStrings::CreateStatic(&IAssetManagerEditorModule::GeneratePrimaryAssetTypeComboBoxStrings, bAllowClear, false))
	];
}

void SPrimaryAssetTypeGraphPin::Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj)
{
	SGraphPin::Construct(SGraphPin::FArguments(), InGraphPinObj);
}

TSharedRef<SWidget>	SPrimaryAssetTypeGraphPin::GetDefaultValueWidget()
{
	FString DefaultString = GraphPinObj->GetDefaultAsString();
	CurrentType = FPrimaryAssetType(*DefaultString);

	return SNew(SVerticalBox)
		.Visibility(this, &SGraphPin::GetDefaultValueVisibility)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			IAssetManagerEditorModule::MakePrimaryAssetTypeSelector(
				FOnGetPrimaryAssetDisplayText::CreateSP(this, &SPrimaryAssetTypeGraphPin::GetDisplayText),
				FOnSetPrimaryAssetType::CreateSP(this, &SPrimaryAssetTypeGraphPin::OnTypeSelected),
				true)
		];
}

void SPrimaryAssetTypeGraphPin::OnTypeSelected(FPrimaryAssetType AssetType)
{
	CurrentType = AssetType;

	if (CurrentType.ToString() != GraphPinObj->GetDefaultAsString())
	{
		const FScopedTransaction Transaction(NSLOCTEXT("GraphEditor", "ChangePinValue", "Change Pin Value"));
		GraphPinObj->Modify();
		GraphPinObj->GetSchema()->TrySetDefaultValue(*GraphPinObj, CurrentType.ToString());
	}
}

FText SPrimaryAssetTypeGraphPin::GetDisplayText() const
{
	return FText::AsCultureInvariant(CurrentType.ToString());
}

#undef LOCTEXT_NAMESPACE
