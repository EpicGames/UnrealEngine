// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IPropertyTypeCustomization.h"
#include "SGraphPin.h"
#include "UObject/PrimaryAssetId.h"

class IPropertyHandle;

/** Primary asset type, reads the list from AssetManager */
class FPrimaryAssetTypeCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance()
	{
		return MakeShareable(new FPrimaryAssetTypeCustomization);
	}

	/** IPropertyTypeCustomization interface */
	virtual void CustomizeHeader(TSharedRef<class IPropertyHandle> StructPropertyHandle, class FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> InStructPropertyHandle, class IDetailChildrenBuilder& ChildBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}

private:

	/** Handle to the struct property being customized */
	TSharedPtr<IPropertyHandle> StructPropertyHandle;

	/** Creates the string list */
	void GenerateComboBoxStrings(TArray< TSharedPtr<FString> >& OutComboBoxStrings, TArray<TSharedPtr<class SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems);

};

/** Graph pin version of UI */
class SPrimaryAssetTypeGraphPin : public SGraphPin
{
public:
	SLATE_BEGIN_ARGS(SPrimaryAssetTypeGraphPin) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphPin* InGraphPinObj);

	//~ Begin SGraphPin Interface
	virtual TSharedRef<SWidget>	GetDefaultValueWidget() override;
	//~ End SGraphPin Interface

private:

	void OnTypeSelected(FPrimaryAssetType AssetType);
	FText GetDisplayText() const;

	FPrimaryAssetType CurrentType;
};
