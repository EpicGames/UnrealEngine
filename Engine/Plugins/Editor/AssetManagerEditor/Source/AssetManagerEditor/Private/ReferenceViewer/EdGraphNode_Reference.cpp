// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceViewer/EdGraphNode_Reference.h"

#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "Containers/VersePath.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "EdGraph/EdGraphPin.h"
#include "HAL/PlatformFileManager.h"
#include "Misc/PackageName.h"
#include "SReferenceViewer.h"
#include "AssetToolsModule.h"
#include "CollectionManagerModule.h"
#include "IAssetTools.h"
#include "IAssetTypeActions.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "ICollectionSource.h"
#include "ReferenceViewer/EdGraph_ReferenceViewer.h"
#include "Styling/AppStyle.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphNode_Reference)

#define LOCTEXT_NAMESPACE "ReferenceViewer"

//////////////////////////////////////////////////////////////////////////
// UEdGraphNode_Reference

UEdGraphNode_Reference::UEdGraphNode_Reference(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	DependencyPin = NULL;
	ReferencerPin = NULL;
	bIsCollapsed = false;
	bIsPackage = false;
	bIsPrimaryAsset = false;
	bUsesThumbnail = false;
	bAllowThumbnail = true;
	AssetTypeColor = FLinearColor(0.55f, 0.55f, 0.55f);
	bIsFiltered = false;
	bIsOverflow = false;
}

void UEdGraphNode_Reference::SetupReferenceNode(const FIntPoint& NodeLoc, const TArray<FAssetIdentifier>& NewIdentifiers, const FAssetData& InAssetData, bool bInAllowThumbnail, bool bInIsADuplicate)
{
	check(NewIdentifiers.Num() > 0);

	NodePosX = NodeLoc.X;
	NodePosY = NodeLoc.Y;

	Identifiers = NewIdentifiers;
	const FAssetIdentifier& First = NewIdentifiers[0];
	FString MainAssetName = InAssetData.AssetName.ToString();
	FText AssetTypeDisplayName;

	FAssetToolsModule& AssetToolsModule = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools"));	
	if (UClass* AssetClass = InAssetData.GetClass())
	{
		AssetTypeDisplayName = AssetClass->GetDisplayNameText();

		if (TSharedPtr<IAssetTypeActions> AssetTypeActions = AssetToolsModule.Get().GetAssetTypeActionsForClass(AssetClass).Pin())
		{
			AssetTypeColor = AssetTypeActions->GetTypeColor();
		}
	}
	else if(InAssetData.IsValid())
	{
		AssetTypeDisplayName = FText::AsCultureInvariant(InAssetData.AssetClassPath.GetAssetName().ToString());
	}
	if (InAssetData.IsValid())
	{
		AssetBrush = FSlateIcon("EditorStyle", FName(*("ClassIcon." + InAssetData.AssetClassPath.GetAssetName().ToString())));
	}

	bIsCollapsed = false;
	bIsPackage = true;
	bAllowThumbnail = bInAllowThumbnail;
	bIsADuplicate = bInIsADuplicate;

	FPrimaryAssetId PrimaryAssetID = NewIdentifiers[0].GetPrimaryAssetId();
	if (PrimaryAssetID.IsValid())  // Management References (PrimaryAssetIDs)
	{
		static const FText ManagerText = LOCTEXT("ReferenceManager", "Manager");
		MainAssetName = PrimaryAssetID.PrimaryAssetType.ToString() + TEXT(":") + PrimaryAssetID.PrimaryAssetName.ToString();
		AssetTypeDisplayName = ManagerText;
		bIsPackage = false;
		bIsPrimaryAsset = true;
	}
	else if (First.IsValue()) // Searchable Names (GamePlay Tags, Data Table Row Handle)
	{
		MainAssetName = First.ValueName.ToString();
		if (AssetTypeDisplayName.IsEmpty())
		{
			static const FName GameplayTagTypeName = "GameplayTag";
			if (First.ObjectName == GameplayTagTypeName)
			{
				static const FText GameplayTagText = LOCTEXT("ReferenceGameplayTag", "Gameplay Tag");
				AssetTypeDisplayName = GameplayTagText;
			}
			else
			{
				AssetTypeDisplayName = FText::AsCultureInvariant(First.ObjectName.ToString());
			}
		}
		else
		{
			AssetTypeDisplayName = FText::FormatNamed(
				LOCTEXT("InAssetFmt", "In {AssetType}: {AssetName}"), 
				TEXT("AssetType"), AssetTypeDisplayName,
				TEXT("AssetName"), FText::AsCultureInvariant(First.ObjectName.ToString())
				);
		}

		bIsPackage = false;
	}
	else if (First.IsPackage() && !InAssetData.IsValid()) 
	{
		const FString PackageNameStr = Identifiers[0].PackageName.ToString();
		if ( PackageNameStr.StartsWith(TEXT("/Script")) )// C++ Packages (/Script Code)
		{
			static const FText ScriptText = LOCTEXT("ReferenceScript", "Script");
			MainAssetName = PackageNameStr.RightChop(8);
			AssetTypeDisplayName = ScriptText;
		}
	}

	if (NewIdentifiers.Num() == 1 )
	{
		static const FName NAME_ActorLabel(TEXT("ActorLabel"));
		InAssetData.GetTagValue(NAME_ActorLabel, MainAssetName); 

		// append the type so it shows up on the extra line
		static const FTextFormat NodeTitleFmt = INVTEXT("{0}\n{1}");
		NodeTitle = FText::Format(NodeTitleFmt, FText::FromString(MainAssetName), AssetTypeDisplayName);
	}
	else
	{
		static const FTextFormat NodeTitleFmt = LOCTEXT("ReferenceNodeMultiplePackagesComment", "{0} and {1} {1}|plural(one=other,other=others)");
		NodeTitle = FText::Format(NodeTitleFmt, FText::FromString(MainAssetName), NewIdentifiers.Num() - 1);
	}
	
	CacheAssetData(InAssetData);
	UpdatePath();
	AllocateDefaultPins();
}

void UEdGraphNode_Reference::SetReferenceNodeCollapsed(const FIntPoint& NodeLoc, int32 InNumReferencesExceedingMax, const TArray<FAssetIdentifier>& NewIdentifiers)
{
	NodePosX = NodeLoc.X;
	NodePosY = NodeLoc.Y;

	Identifiers = NewIdentifiers;
	bIsCollapsed = true;
	bUsesThumbnail = false;
	bIsOverflow = true;
	AssetBrush = FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.WarningWithColor");

	NodeTitle = FText::Format( LOCTEXT("ReferenceNodeCollapsedTitle", "{0}"), FText::AsNumber(InNumReferencesExceedingMax));

	CacheAssetData(FAssetData());
	UpdatePath();
	AllocateDefaultPins();
}

void UEdGraphNode_Reference::AddReferencer(UEdGraphNode_Reference* ReferencerNode)
{
	UEdGraphPin* ReferencerDependencyPin = ReferencerNode->GetDependencyPin();

	if ( ensure(ReferencerDependencyPin) )
	{
		ReferencerDependencyPin->bHidden = false;
		ReferencerPin->bHidden = false;
		ReferencerPin->MakeLinkTo(ReferencerDependencyPin);
	}
}

FAssetIdentifier UEdGraphNode_Reference::GetIdentifier() const
{
	if (Identifiers.Num() > 0)
	{
		return Identifiers[0];
	}

	return FAssetIdentifier();
}

void UEdGraphNode_Reference::GetAllIdentifiers(TArray<FAssetIdentifier>& OutIdentifiers) const
{
	OutIdentifiers.Append(Identifiers);
}

void UEdGraphNode_Reference::GetAllPackageNames(TArray<FName>& OutPackageNames) const
{
	for (const FAssetIdentifier& AssetId : Identifiers)
	{
		if (AssetId.IsPackage())
		{
			OutPackageNames.AddUnique(AssetId.PackageName);
		}
	}
}

UEdGraph_ReferenceViewer* UEdGraphNode_Reference::GetReferenceViewerGraph() const
{
	return Cast<UEdGraph_ReferenceViewer>( GetGraph() );
}

FText UEdGraphNode_Reference::GetNodeTitle(ENodeTitleType::Type TitleType) const
{
	return NodeTitle;
}

FLinearColor UEdGraphNode_Reference::GetNodeTitleColor() const
{
	if (bIsPrimaryAsset)
	{
		return FLinearColor(0.2f, 0.8f, 0.2f);
	}
	else if (bIsPackage)
	{
		return AssetTypeColor;
	}
	else if (bIsCollapsed)
	{
		return FLinearColor(0.55f, 0.55f, 0.55f);
	}
	else 
	{
		return FLinearColor(0.0f, 0.55f, 0.62f);
	}
}

FText UEdGraphNode_Reference::GetTooltipText() const
{
	if (Identifiers.IsEmpty())
	{
		return FText::GetEmpty();
	}

	// Showing up to 15 nodes paths, in order to avoid an extremely long tooltip and huge widget
	constexpr int32 MaxReferenceNum = 15;

	FString TooltipString;

	if (IsCollapsed())
	{
		TooltipString.Append(GetNodeTitle(ENodeTitleType::FullTitle).ToString() + " collapsed nodes:\n");
	}

	const bool bIsShowingContentVersePath = GetReferenceViewerGraph()->IsShowingContentVersePath();
	const bool bShowCollections = !IsCollapsed() && bIsPackage;

	auto IsPackage = [](const FAssetIdentifier& AssetId)
	{
		return !AssetId.GetPrimaryAssetId().IsValid() && !AssetId.IsValue();
	};

	// Get the asset data we need once.
	TArray<FName> PackageNames;
	if (bIsShowingContentVersePath)
	{
		PackageNames.Reserve(FMath::Min(Identifiers.Num(), MaxReferenceNum));

		int32 ReferenceCount = 0;
		for (const FAssetIdentifier& AssetId : Identifiers)
		{
			if (IsPackage(AssetId))
			{
				PackageNames.Add(AssetId.PackageName);
			}

			if (++ReferenceCount > MaxReferenceNum)
			{
				// We only need the asset data for the first MaxReferenceNum identifiers since that is all we show.
				break;
			}
		}

		PackageNames.Sort(FNameFastLess());
		PackageNames.SetNum(Algo::Unique(PackageNames));
	}
	else if (bShowCollections)
	{
		PackageNames.Add(Identifiers[0].PackageName);
	}

	TMap<FName, FAssetData> Assets;
	UE::AssetRegistry::GetAssetForPackages(PackageNames, Assets);

	{
		int32 ReferenceCount = 0;
		for (const FAssetIdentifier& AssetId : Identifiers)
		{
			if (!TooltipString.IsEmpty())
			{
				TooltipString.Append(TEXT("\n"));
			}

			UE::Core::FVersePath VersePath;

			if (bIsShowingContentVersePath && IsPackage(AssetId))
			{
				if (const FAssetData* AssetData = Assets.Find(AssetId.PackageName))
				{
					VersePath = AssetData->GetVersePath();
				}
			}

			if (VersePath.IsValid())
			{
				TooltipString.Append(VersePath.ToString());
			}
			else
			{
				TooltipString.Append(AssetId.ToString());
			}

			// Avoiding an extremely long tooltip
			if (++ReferenceCount > MaxReferenceNum)
			{
				TooltipString.Append(TEXT("\n..."));
				break;
			}
		}
	}

	if (bShowCollections)
	{
		const FAssetIdentifier& AssetId = Identifiers[0];

		// Retrieve Collections information
		if (const FAssetData* AssetData = Assets.Find(AssetId.PackageName))
		{
			ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();

			TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
			CollectionManager.GetVisibleCollectionContainers(CollectionContainers);

			TArray<FCollectionNameType> ObjectCollections;
			for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
			{
				ObjectCollections.Reset();
				CollectionContainer->GetCollectionsContainingObject(AssetData->ToSoftObjectPath(), ObjectCollections);

				if (!ObjectCollections.IsEmpty())
				{
					Algo::Sort(ObjectCollections, [](const FCollectionNameType& A, const FCollectionNameType& B)
						{
							return A.Name.LexicalLess(B.Name);
						});
					const int32 EndIndex = Algo::Unique(ObjectCollections, [](const FCollectionNameType& A, const FCollectionNameType& B)
						{
							return A.Name == B.Name;
						});
					ObjectCollections.RemoveAt(EndIndex, ObjectCollections.Num() - EndIndex, EAllowShrinking::No);

					TooltipString.Append(TEXT("\n\n"));
					TooltipString.Append(CollectionContainer->GetCollectionSource()->GetTitle().ToString());
					TooltipString.Append(TEXT(":"));
					for (int32 i = 0; i < ObjectCollections.Num(); i++)
					{
						if (i > 0)
						{
							TooltipString.Append(TEXT(","));
						}

						TooltipString.Append(TEXT(" ") + ObjectCollections[i].Name.ToString());
					}
				}
			}
		}
	}

	return FText::FromString(MoveTemp(TooltipString));
}

FSlateIcon UEdGraphNode_Reference::GetIconAndTint(FLinearColor& OutColor) const
{
	OutColor = bIsOverflow ? FLinearColor::White : AssetTypeColor;
	return AssetBrush;
}

void UEdGraphNode_Reference::AllocateDefaultPins()
{
	ReferencerPin = CreatePin( EEdGraphPinDirection::EGPD_Input, NAME_None, NAME_None);
	DependencyPin = CreatePin( EEdGraphPinDirection::EGPD_Output, NAME_None, NAME_None);

	ReferencerPin->bHidden = true;
	FName PassiveName = ::GetName(EDependencyPinCategory::LinkEndPassive);
	ReferencerPin->PinType.PinCategory = PassiveName;
	DependencyPin->bHidden = true;
	DependencyPin->PinType.PinCategory = PassiveName;
}

UObject* UEdGraphNode_Reference::GetJumpTargetForDoubleClick() const
{
	if (Identifiers.Num() > 0 )
	{
		GetReferenceViewerGraph()->SetGraphRoot(Identifiers, FIntPoint(NodePosX, NodePosY));
		GetReferenceViewerGraph()->RebuildGraph();
	}
	return NULL;
}

UEdGraphPin* UEdGraphNode_Reference::GetDependencyPin()
{
	return DependencyPin;
}

UEdGraphPin* UEdGraphNode_Reference::GetReferencerPin()
{
	return ReferencerPin;
}

void UEdGraphNode_Reference::UpdatePath()
{
	if (!IsCollapsed() && Identifiers.Num() == 1)
	{
		UE::Core::FVersePath VersePath;

		if (bIsPackage && CachedAssetData.IsValid() && GetReferenceViewerGraph()->IsShowingContentVersePath())
		{
			VersePath = CachedAssetData.GetVersePath();
		}

		if (VersePath.IsValid())
		{
			NodeComment = MoveTemp(VersePath).ToString();
		}
		else
		{
			NodeComment = Identifiers[0].PackageName.ToString();
		}
	}
	else
	{
		NodeComment.Reset();
	}
}

void UEdGraphNode_Reference::CacheAssetData(const FAssetData& AssetData)
{
	if ( AssetData.IsValid() && IsPackage() )
	{
		bUsesThumbnail = true;
		CachedAssetData = AssetData;
	}
	else
	{
		CachedAssetData = FAssetData();
		bUsesThumbnail = false;

		if (Identifiers.Num() == 1 )
		{
			const FString PackageNameStr = Identifiers[0].PackageName.ToString();
			if ( FPackageName::IsValidLongPackageName(PackageNameStr, true) )
			{
				if ( PackageNameStr.StartsWith(TEXT("/Script")) )
				{
					// Used Only in the UI for the Thumbnail
					CachedAssetData.AssetClassPath = FTopLevelAssetPath(TEXT("/EdGraphNode_Reference"), TEXT("Code"));
				}
				else
				{
					const FString PotentiallyMapFilename = FPackageName::LongPackageNameToFilename(PackageNameStr, FPackageName::GetMapPackageExtension());
					const bool bIsMapPackage = FPlatformFileManager::Get().GetPlatformFile().FileExists(*PotentiallyMapFilename);
					if ( bIsMapPackage )
					{
						// Used Only in the UI for the Thumbnail
						CachedAssetData.AssetClassPath = TEXT("/Script/Engine.World");
					}
				}
			}
		}
		else
		{
			CachedAssetData.AssetClassPath = FTopLevelAssetPath(TEXT("/EdGraphNode_Reference"), TEXT("Multiple Nodes"));
		}
	}

}

const FAssetData& UEdGraphNode_Reference::GetAssetData() const
{
	return CachedAssetData;
}

bool UEdGraphNode_Reference::AllowsThumbnail() const
{
	return bAllowThumbnail;
}

bool UEdGraphNode_Reference::UsesThumbnail() const
{
	return bUsesThumbnail;
}

bool UEdGraphNode_Reference::IsPackage() const
{
	return bIsPackage;
}

bool UEdGraphNode_Reference::IsCollapsed() const
{
	return bIsCollapsed;
}

void UEdGraphNode_Reference::SetIsFiltered(bool bInFiltered)
{
	bIsFiltered = bInFiltered;	
}

bool UEdGraphNode_Reference::GetIsFiltered() const
{
	return bIsFiltered;
}

#undef LOCTEXT_NAMESPACE

