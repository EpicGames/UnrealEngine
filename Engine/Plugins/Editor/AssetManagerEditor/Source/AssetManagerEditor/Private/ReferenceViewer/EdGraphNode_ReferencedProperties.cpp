// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceViewer/EdGraphNode_ReferencedProperties.h"
#include "ReferenceViewer/EdGraphNode_Reference.h"
#include "ReferenceViewer/ReferenceViewerSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraphNode_ReferencedProperties)

TSharedRef<FReferencingPropertyDescription> FReferencingPropertyDescription::MakeSharedPropertyDescription(
	const FReferencingPropertyDescription& InPropertyDescription
)
{
	return MakeShared<FReferencingPropertyDescription>(InPropertyDescription);
}

FString FReferencingPropertyDescription::GetTypeAsString() const
{
	FString TypeString = "";

	switch (Type)
	{
	case EAssetReferenceType::Property:
		TypeString = TEXT("Property Type");
		break;
	case EAssetReferenceType::Component:
		TypeString =  TEXT("Component Type");
		break;
	case EAssetReferenceType::Value:
		TypeString =  TEXT("Property Value");
		break;
	case EAssetReferenceType::None:
		TypeString =  TEXT("");
		break;
	default: ;
	}

	return TypeString;
}

UObject* UEdGraphNode_ReferencedProperties::GetReferencingObject() const
{
	if (ReferencingNode)
	{
		return ReferencingNode->GetAssetData().GetAsset();
	}

	return nullptr;
}

UObject* UEdGraphNode_ReferencedProperties::GetReferencedObject() const
{
	if (ReferencedNode)
	{
		return ReferencedNode->GetAssetData().GetAsset();
	}

	return nullptr;
}

void UEdGraphNode_ReferencedProperties::SetupReferencedPropertiesNode(const TArray<FReferencingPropertyDescription>& InPropertiesDescription
	, const TObjectPtr<UEdGraphNode_Reference>& InReferencingNode, const TObjectPtr<UEdGraphNode_Reference>& InReferencedNode)
{
	ReferencedPropertyDescription.Empty();
	for (const FReferencingPropertyDescription& PropertyDescription : InPropertiesDescription)
	{
		ReferencedPropertyDescription.Add(FReferencingPropertyDescription::MakeSharedPropertyDescription(PropertyDescription));
	}

	ReferencingNode = InReferencingNode;
	ReferencedNode = InReferencedNode;

	if (OnPropertiesDescriptionUpdated().IsBound())
	{
		OnPropertiesDescriptionUpdated().Broadcast();
	}

	RefreshLocation();
}

void UEdGraphNode_ReferencedProperties::RefreshLocation(const FVector2f& InNodeSize)
{
	if (!ReferencedNode || !ReferencingNode)
	{
		return;
	}

	int32 NodeX = (ReferencedNode->NodePosX + ReferencingNode->NodePosX) * 0.5f;
	int32 NodeY = (ReferencedNode->NodePosY + ReferencingNode->NodePosY) * 0.5f;

	if (const UReferenceViewerSettings* Settings = GetDefault<UReferenceViewerSettings>())
	{
		int32 NodeSizeY = Settings->IsCompactMode() ? 100 : 200;
		NodeSizeY += Settings->IsShowPath() ? 40 : 0;
		NodeY += NodeSizeY * 0.5f;
	}

	NodeX -= (InNodeSize.X - 128) * 0.5f;
	NodeY -= InNodeSize.Y * 0.5f;

	NodePosX = NodeX;
	NodePosY = NodeY;

	// TODO: This positioning is quite rough, if design stays the same, we could improve it in the future
}
