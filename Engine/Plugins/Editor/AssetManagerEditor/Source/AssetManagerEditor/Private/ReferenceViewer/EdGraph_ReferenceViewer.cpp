// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceViewer/EdGraph_ReferenceViewer.h"
#include "Misc/FilterCollection.h"
#include "ReferenceViewer/EdGraphNode_Reference.h"
#include "Misc/IFilter.h"
#include "Misc/ScopedSlowTask.h"
#include "ReferenceViewer/ReferenceViewerSettings.h"
#include "EdGraph/EdGraphPin.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetThumbnail.h"
#include "SReferenceViewer.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "CollectionManagerModule.h"
#include "AssetManagerEditorModule.h"
#include "Engine/AssetManager.h"
#include "Engine/SCS_Node.h"
#include "Engine/SimpleConstructionScript.h"
#include "StructUtils/UserDefinedStruct.h"
#include "ReferenceViewer/EdGraphNode_ReferencedProperties.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EdGraph_ReferenceViewer)

#define LOCTEXT_NAMESPACE "EdGraph_ReferenceViewer"

FReferenceNodeInfo::FReferenceNodeInfo(const FAssetIdentifier& InAssetId, bool InbReferencers)
	: AssetId(InAssetId)
	, bReferencers(InbReferencers)
	, bIsRedirector(false)
	, OverflowCount(0)
	, bExpandAllChildren(false)
	, ChildProvisionSize(0)
	, PassedFilters(true)
{}

bool FReferenceNodeInfo::IsFirstParent(const FAssetIdentifier& InParentId) const
{
	return Parents.IsEmpty() || Parents[0] == InParentId;
}

bool FReferenceNodeInfo::IsRedirector() const
{
	return bIsRedirector;
}

bool FReferenceNodeInfo::IsADuplicate() const
{
	return Parents.Num() > 1;
}

int32 FReferenceNodeInfo::ProvisionSize(const FAssetIdentifier& InParentId) const
{
	return IsFirstParent(InParentId) ? ChildProvisionSize : 1;
}

UEdGraph_ReferenceViewer::UEdGraph_ReferenceViewer(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bShowingContentVersePath(false)
{
	if (!IsTemplate())
	{
		AssetThumbnailPool = MakeShareable( new FAssetThumbnailPool(1024) );
	}

	Settings = GetMutableDefault<UReferenceViewerSettings>();
}

void UEdGraph_ReferenceViewer::BeginDestroy()
{
	AssetThumbnailPool.Reset();

	Super::BeginDestroy();
}

void UEdGraph_ReferenceViewer::SetGraphRoot(const TArray<FAssetIdentifier>& GraphRootIdentifiers, const FIntPoint& GraphRootOrigin)
{
	CurrentGraphRootIdentifiers = GraphRootIdentifiers;
	CurrentGraphRootOrigin = GraphRootOrigin;

	// If we're focused on a searchable name, enable that flag
	for (const FAssetIdentifier& AssetId : GraphRootIdentifiers)
	{
		if (AssetId.IsValue())
		{
			Settings->SetShowSearchableNames(true);
		}
		else if (AssetId.GetPrimaryAssetId().IsValid())
		{
			UAssetManager::Get().UpdateManagementDatabase();
			Settings->SetShowManagementReferencesEnabled(true);
		}
	}
}

const TArray<FAssetIdentifier>& UEdGraph_ReferenceViewer::GetCurrentGraphRootIdentifiers() const
{
	return CurrentGraphRootIdentifiers;
}

TWeakPtr<SReferenceViewer> UEdGraph_ReferenceViewer::GetReferenceViewer() const
{
	return ReferenceViewer;
}

void UEdGraph_ReferenceViewer::SetReferenceViewer(TSharedPtr<SReferenceViewer> InViewer)
{
	ReferenceViewer = InViewer;
}

bool UEdGraph_ReferenceViewer::GetSelectedAssetsForMenuExtender(const class UEdGraphNode* Node, TArray<FAssetIdentifier>& SelectedAssets) const
{
	if (!ReferenceViewer.IsValid())
	{
		return false;
	}
	TSharedPtr<SGraphEditor> GraphEditor = ReferenceViewer.Pin()->GetGraphEditor();

	if (!GraphEditor.IsValid())
	{
		return false;
	}

	TSet<UObject*> SelectedNodes = GraphEditor->GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
	{
		if (UEdGraphNode_Reference* ReferenceNode = Cast<UEdGraphNode_Reference>(*It))
		{
			if (!ReferenceNode->IsCollapsed())
			{
				SelectedAssets.Add(ReferenceNode->GetIdentifier());
			}
		}
	}
	return true;
}

UEdGraphNode_Reference* UEdGraph_ReferenceViewer::RebuildGraph()
{
	RemoveAllNodes();
	UEdGraphNode_Reference* NewRootNode = nullptr;

	if (Settings->GetFindPathEnabled())
	{
		NewRootNode = FindPath(CurrentGraphRootIdentifiers[0], TargetIdentifier);
	}
	else
	{
		NewRootNode = ConstructNodes(CurrentGraphRootIdentifiers, CurrentGraphRootOrigin);
	}

	return NewRootNode;
}

void UEdGraph_ReferenceViewer::SetShowingContentVersePath(bool bInShowingContentVersePath)
{
	if (bShowingContentVersePath != bInShowingContentVersePath)
	{
		bShowingContentVersePath = bInShowingContentVersePath;

		UpdatePaths();
	}
}

void UEdGraph_ReferenceViewer::UpdatePaths()
{
	for (const TObjectPtr<UEdGraphNode>& Node : Nodes)
	{
		if (UEdGraphNode_Reference* ReferenceNode = Cast<UEdGraphNode_Reference>(Node))
		{
			ReferenceNode->UpdatePath();
		}
	}
}

void UEdGraph_ReferenceViewer::SetIsAssetIdentifierPassingSearchFilterCallback(const TOptional<FIsAssetIdentifierPassingSearchFilterCallback>& InIsAssetIdentifierPassingSearchFilterCallback)
{
	if (InIsAssetIdentifierPassingSearchFilterCallback.IsSet())
	{
		DoesAssetPassSearchFilterCallback = [IsAssetIdentifierPassingSearchFilterCallback = InIsAssetIdentifierPassingSearchFilterCallback.GetValue()](const FAssetIdentifier& InAssetIdentifier, const FAssetData& InAssetData)
		{
			return IsAssetIdentifierPassingSearchFilterCallback(InAssetIdentifier);
		};
	}
	else
	{
		DoesAssetPassSearchFilterCallback.Reset();
	}
}

FName UEdGraph_ReferenceViewer::GetCurrentCollectionFilter() const
{
	return CurrentCollectionFilterName;
}

void UEdGraph_ReferenceViewer::GetCurrentCollectionFilter(ICollectionContainer*& OutCollectionContainer, FName& OutCollectionName) const
{
	OutCollectionContainer = CurrentCollectionFilterContainer.Get();
	OutCollectionName = CurrentCollectionFilterName;
}

void UEdGraph_ReferenceViewer::SetCurrentCollectionFilter(FName NewFilter)
{
	SetCurrentCollectionFilter(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(), NewFilter);
}

void UEdGraph_ReferenceViewer::SetCurrentCollectionFilter(const TSharedPtr<ICollectionContainer>& CollectionContainer, FName CollectionName)
{
	CurrentCollectionFilterContainer = CollectionContainer;
	CurrentCollectionFilterName = CollectionName;
}

TArray<FName> UEdGraph_ReferenceViewer::GetCurrentPluginFilter() const
{
	return CurrentPluginFilter;
}

void UEdGraph_ReferenceViewer::SetCurrentPluginFilter(TArray<FName> NewFilter)
{
	CurrentPluginFilter = NewFilter;
}

TArray<FName> UEdGraph_ReferenceViewer::GetEncounteredPluginsAmongNodes() const
{
	return EncounteredPluginsAmongNodes;
}


void UEdGraph_ReferenceViewer::SetCurrentFilterCollection(TSharedPtr< TFilterCollection< FReferenceNodeInfo& > > InFilterCollection )
{
	FilterCollection = InFilterCollection;
}

FAssetManagerDependencyQuery UEdGraph_ReferenceViewer::GetReferenceSearchFlags(bool bHardOnly) const
{
	using namespace UE::AssetRegistry;
	FAssetManagerDependencyQuery Query;
	Query.Categories = EDependencyCategory::None;
	Query.Flags = EDependencyQuery::NoRequirements;

	bool bLocalIsShowSoftReferences = Settings->IsShowSoftReferences() && !bHardOnly;
	if (bLocalIsShowSoftReferences || Settings->IsShowHardReferences())
	{
		Query.Categories |= EDependencyCategory::Package;
		Query.Flags |= bLocalIsShowSoftReferences ? EDependencyQuery::NoRequirements : EDependencyQuery::Hard;
		Query.Flags |= Settings->IsShowHardReferences() ? EDependencyQuery::NoRequirements : EDependencyQuery::Soft;
		switch (Settings->GetEditorOnlyReferenceFilterType())
		{
		case EEditorOnlyReferenceFilterType::Game: Query.Flags |= EDependencyQuery::Game; break;
		case EEditorOnlyReferenceFilterType::Propagation: Query.Flags |= EDependencyQuery::Propagation; break;
		case EEditorOnlyReferenceFilterType::EditorOnly: [[fallthrough]];
		default: /* No requirements */ ; break;
		}
	}
	if (Settings->IsShowSearchableNames() && !bHardOnly)
	{
		Query.Categories |= EDependencyCategory::SearchableName;
	}
	if (Settings->IsShowManagementReferences())
	{
		Query.Categories |= EDependencyCategory::Manage;
		Query.Flags |= bHardOnly ? EDependencyQuery::Direct : EDependencyQuery::NoRequirements;
	}

	return Query;
}

UEdGraphNode_Reference* UEdGraph_ReferenceViewer::ConstructNodes(const TArray<FAssetIdentifier>& GraphRootIdentifiers, const FIntPoint& GraphRootOrigin )
{
	if (GraphRootIdentifiers.Num() > 0)
	{
		// It both were false, nothing (other than the GraphRootIdentifiers) would be displayed
		check(Settings->IsShowReferencers() || Settings->IsShowDependencies());

		// Refresh the current collection filter
		CurrentCollectionPackages.Empty();
		if (ShouldFilterByCollection())
		{
			TArray<FSoftObjectPath> AssetPaths;
			CurrentCollectionFilterContainer->GetAssetsInCollection(CurrentCollectionFilterName, ECollectionShareType::CST_All, AssetPaths);

			CurrentCollectionPackages.Reserve(AssetPaths.Num());
			for (const FSoftObjectPath& AssetPath : AssetPaths)
			{
				CurrentCollectionPackages.Add(AssetPath.GetLongPackageFName());
			}
		}

		// Prepare for plugin filtering.
		{
			// Collect plugin names from assets reachable in the graph if the graph had been unfiltered.
			EncounteredPluginsAmongNodes.Empty();
			GetUnfilteredGraphPluginNames(GraphRootIdentifiers, EncounteredPluginsAmongNodes);

			// Remove plugins from the current filter that were not encountered in the new unfiltered graph.
			for (TArray<FName>::TIterator It(CurrentPluginFilter); It; ++It)
			{
				if (!EncounteredPluginsAmongNodes.Contains(*It))
				{
					It.RemoveCurrent();
				}
			}
		}

		// Create & Populate the NodeInfo Maps 
		// Note to add an empty parent to the root so that if the root node again gets found again as a duplicate, that next parent won't be 
		// identified as the primary root and also it will appear as having multiple parents.
		TMap<FAssetIdentifier, FReferenceNodeInfo> NewReferenceNodeInfos;
		for (const FAssetIdentifier& RootIdentifier : GraphRootIdentifiers)
		{
			FReferenceNodeInfo& RootNodeInfo = NewReferenceNodeInfos.FindOrAdd(RootIdentifier, FReferenceNodeInfo(RootIdentifier, true));
			RootNodeInfo.Parents.Emplace(FAssetIdentifier(NAME_None));
		}
		if (!Settings->GetFindPathEnabled())
		{
			RecursivelyPopulateNodeInfos(true, GraphRootIdentifiers, NewReferenceNodeInfos, 0, Settings->GetSearchReferencerDepthLimit());
		}

		TMap<FAssetIdentifier, FReferenceNodeInfo> NewDependencyNodeInfos;
		for (const FAssetIdentifier& RootIdentifier : GraphRootIdentifiers)
		{
			FReferenceNodeInfo& DRootNodeInfo = NewDependencyNodeInfos.FindOrAdd(RootIdentifier, FReferenceNodeInfo(RootIdentifier, false));
			DRootNodeInfo.Parents.Emplace(FAssetIdentifier(NAME_None));
		}
		if (!Settings->GetFindPathEnabled())
		{
			RecursivelyPopulateNodeInfos(false, GraphRootIdentifiers, NewDependencyNodeInfos, 0, Settings->GetSearchDependencyDepthLimit());
		}

		// Store the AssetData in the NodeInfos if needed, and collect Asset Type UClasses to populate the filters
		TSet<FTopLevelAssetPath> AllClasses;
		for (TPair<FAssetIdentifier, FReferenceNodeInfo>&  InfoPair : NewReferenceNodeInfos)
		{
			// Make sure AssetData is valid
			if (!InfoPair.Value.AssetData.IsValid())
			{
				const FName& PackageName = InfoPair.Key.PackageName;
				TMap<FName, FAssetData> PackageToAssetDataMap;
				UE::AssetRegistry::GetAssetForPackages({PackageName}, PackageToAssetDataMap);
				InfoPair.Value.AssetData = PackageToAssetDataMap.FindRef(PackageName);
			}

			AllClasses.Add(InfoPair.Value.AssetData.AssetClassPath);
		}

		for (TPair<FAssetIdentifier, FReferenceNodeInfo>&  InfoPair : NewDependencyNodeInfos)
		{
			// Make sure AssetData is valid
			if (!InfoPair.Value.AssetData.IsValid())
			{
				const FName& PackageName = InfoPair.Key.PackageName;
				TMap<FName, FAssetData> PackageToAssetDataMap;
				UE::AssetRegistry::GetAssetForPackages({PackageName}, PackageToAssetDataMap);
				InfoPair.Value.AssetData = PackageToAssetDataMap.FindRef(PackageName);
			}

			AllClasses.Add(InfoPair.Value.AssetData.AssetClassPath);
		}

		// Update the cached class types list
		CurrentClasses = AllClasses;
		OnAssetsChangedDelegate.Broadcast();

		ReferencerNodeInfos = NewReferenceNodeInfos;
		DependencyNodeInfos = NewDependencyNodeInfos;
	}
	else
	{
		ReferencerNodeInfos.Empty();
		DependencyNodeInfos.Empty();
	}

	return RefilterGraph();
}

void UEdGraph_ReferenceViewer::RefreshReferencedPropertiesNode(const UEdGraphNode_ReferencedProperties* InNode)
{
	const TObjectPtr<UEdGraphNode_Reference>& ReferencingNode = InNode->GetReferencingNode();
	if (!ReferencingNode)
	{
		return;
	}

	const TObjectPtr<UEdGraphNode_Reference>& ReferencedNode = InNode->GetReferencedNode();
	if (!ReferencedNode)
	{
		return;
	}

	UObject* ReferencingObject = InNode->GetReferencingObject();
	UObject* ReferencedObject = InNode->GetReferencedObject();
	if (!ReferencingObject || !ReferencedObject)
	{
		return;
	}

	TArray<FReferencingPropertyDescription> ReferencingPropertiesArray =
		RetrieveReferencingProperties(ReferencingObject, ReferencedObject);

	CreateReferencedPropertiesNode(ReferencingPropertiesArray, ReferencingNode, ReferencedNode);
}

void UEdGraph_ReferenceViewer::CloseReferencedPropertiesNode(UEdGraphNode_ReferencedProperties* InNode)
{
	if (InNode)
	{
		uint32 NodesPairHash = GetTypeHash(InNode->GetReferencingNode()) ^ GetTypeHash(InNode->GetReferencedNode());

		if (ReferencedPropertiesNodes.Find(NodesPairHash))
		{
			ReferencedPropertiesNodes.Remove(NodesPairHash);
		}

		RemoveNode(InNode);
	}
}

void UEdGraph_ReferenceViewer::RefreshReferencedPropertiesNodes()
{
	for (const TPair<uint32, TWeakObjectPtr<UEdGraphNode_ReferencedProperties>>& Pair : ReferencedPropertiesNodes)
	{
		if (UEdGraphNode_ReferencedProperties* Node = Pair.Value.Get())
		{
			RefreshReferencedPropertiesNode(Node);
		}
	}
}

TArray<FReferencingPropertyDescription> UEdGraph_ReferenceViewer::RetrieveReferencingProperties(UObject* InReferencer, UObject* InReferencedAsset)
{
	// This method will check InReferencer for references to InReferencedAsset.
	// Search includes property types and values.
	// At this stage, it is possible that some cases won't work well (missing references)
	// On the other end, some results won't be entirely helpful to the user

	if (!InReferencer || !InReferencedAsset)
	{
		return {};
	}

	TArray<FReferencingPropertyDescription> ReferencingProperties;

	// Registering referencing properties to the output array. Property type defaults to EReferencedPropertyType::Property
	auto AddReferencingProperty = [&ReferencingProperties, &InReferencedAsset](const FString& InPropertyName, const FString& InReferencerName, const FString& InReferencedNodeName,
									  FReferencingPropertyDescription::EAssetReferenceType InPropertyType =
										  FReferencingPropertyDescription::EAssetReferenceType::Property,
									  bool bInIndirectReference = false
								  )
	{
		FReferencingPropertyDescription PropertyDescription(
			InPropertyName, InReferencerName, InReferencedNodeName, InPropertyType, InReferencedAsset->GetClass(), bInIndirectReference
		);

		if (!ReferencingProperties.Contains(PropertyDescription))
		{
			ReferencingProperties.AddUnique(PropertyDescription);
		}
	};

	// User Defined Struct ("BP Struct")
	if (UUserDefinedStruct* ReferencerStruct = Cast<UUserDefinedStruct>(InReferencer))
	{
		const FProperty* CurrentStructProperty = ReferencerStruct->PropertyLink;
		while (CurrentStructProperty)
		{
			bool bMatchFound = false;

			if (const FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(CurrentStructProperty))
			{
				if (const TObjectPtr<UClass>& PropertyClass = ObjectProperty->PropertyClass)
				{
					bMatchFound = PropertyClass->ClassGeneratedBy == InReferencedAsset;
				}
			}
			else if (const FByteProperty* ByteProperty = CastField<FByteProperty>(CurrentStructProperty))
			{
				bMatchFound = ByteProperty->Enum == InReferencedAsset;
			}
			else if (const FStructProperty* StructProperty = CastField<FStructProperty>(CurrentStructProperty))
			{
				bMatchFound = StructProperty->Struct == InReferencedAsset;
			}

			if (bMatchFound)
			{
				AddReferencingProperty(CurrentStructProperty->GetDisplayNameText().ToString(), InReferencer->GetName(), InReferencedAsset->GetName());
			}

			CurrentStructProperty = CurrentStructProperty->PropertyLinkNext;
		}

		// We are done with this Asset Struct
		return ReferencingProperties;
	}

	// In case Referencer is a Blueprint, let's look for BP Actor Components referencing the Referenced Asset
	if (UBlueprint* ReferencerBlueprint = Cast<UBlueprint>(InReferencer))
	{
		if (const TObjectPtr<USimpleConstructionScript>& SimpleConstructionScript = ReferencerBlueprint->SimpleConstructionScript)
		{
			const TArray<USCS_Node*>& CDONodes = SimpleConstructionScript->GetAllNodes();
			for (const USCS_Node* Node : CDONodes)
			{
				if (!Node)
				{
					continue;
				}

				UClass* ComponentClass = Node->ComponentClass;
				if (!ComponentClass)
				{
					continue;
				}

				UObject* GeneratingBlueprintObject = ComponentClass->ClassGeneratedBy;
				if (!GeneratingBlueprintObject)
				{
					continue;
				}

				if (GeneratingBlueprintObject == InReferencedAsset)
				{
					// The blueprint used to generate the current CDO Component Node is the same as the referenced asset: add this to output properties names
					AddReferencingProperty(*Node->GetVariableName().ToString(), InReferencer->GetName(), InReferencedAsset->GetName(), FReferencingPropertyDescription::EAssetReferenceType::Component);
				}
			}
		}
	}

	// This string will be used as support to export properties as text, in case we need it
	FString PropertyExportString;

	// Going through available fields
	for (TFieldIterator<FProperty> PropertyIt(InReferencer->GetClass()); PropertyIt; ++PropertyIt)
	{
		if (!PropertyIt)
		{
			continue;
		}

		PropertyExportString.Empty();

		// Blueprint Array
		if (FArrayProperty* ArrayProperty = CastField<FArrayProperty>(*PropertyIt))
		{
			FScriptArrayHelper_InContainer ArrayHelper(ArrayProperty, InReferencer);
			for (int32 ItemIndex = 0; ItemIndex < ArrayHelper.Num(); ItemIndex++)
			{
				uint8* ArrayElementMemory = ArrayHelper.GetRawPtr(ItemIndex);

				// Blueprint Property
				if (ArrayProperty->GetOwnerClass() == UBlueprint::StaticClass())
				{
					// We are looking for Blueprint Variables only
					if (ArrayProperty->GetName() != TEXT("NewVariables"))
					{
						continue;
					}

					const FBPVariableDescription& BPVariableDescription = *reinterpret_cast<const FBPVariableDescription*>(ArrayElementMemory);

					bool bAddProperty = false;
					UObject* SubCategoryObject = BPVariableDescription.VarType.PinSubCategoryObject.Get();
					if (SubCategoryObject == InReferencedAsset)
					{
						bAddProperty = true;
					}
					else if (UClass* BPVariableClass = Cast<UClass>(SubCategoryObject))
					{
						const TObjectPtr<UObject>& GeneratingBlueprintObject = BPVariableClass->ClassGeneratedBy;
						if (GeneratingBlueprintObject == InReferencedAsset)
						{
							bAddProperty = true;
						}
					}
					else if (FProperty* InnerProperty = ArrayProperty->Inner)// todo: can we avoid using ExportTextItem_Direct in this case?
					{
						InnerProperty->ExportTextItem_Direct(PropertyExportString, ArrayHelper.GetRawPtr(ItemIndex), ArrayHelper.GetRawPtr(ItemIndex), InReferencer, PPF_IncludeTransient);
						if (!PropertyExportString.IsEmpty() && PropertyExportString.Contains(InReferencedAsset->GetPathName()))
						{
							bAddProperty = true;
						}
					}

					if (bAddProperty)
					{
						AddReferencingProperty(BPVariableDescription.VarName.ToString(), InReferencer->GetName(), InReferencedAsset->GetName());
					}
				}
				// Other
				else if (const FProperty* InnerProperty = ArrayProperty->Inner)
				{
					if (InnerProperty->IsA<FObjectProperty>())
					{
						UObject* Object;
						InnerProperty->GetValue_InContainer(ArrayElementMemory, &Object);
						if (Object == InReferencedAsset)
						{
							const FString ItemIndexString = "[" + FString::FromInt(ItemIndex) + "]";
							AddReferencingProperty(ArrayProperty->GetFName().ToString() + ItemIndexString, InReferencer->GetName(), InReferencedAsset->GetName());
						}
					}
				}
			}
		}
		// Native Array
		else if (PropertyIt->ArrayDim > 1)
		{
			for (int32 ItemIndex = 0; ItemIndex < PropertyIt->ArrayDim; ItemIndex++)
			{
				PropertyIt->ExportText_InContainer(ItemIndex, PropertyExportString, InReferencer, InReferencer, InReferencer, PPF_IncludeTransient);

				if (!PropertyExportString.IsEmpty() && PropertyExportString.Contains(InReferencedAsset->GetPathName()))
				{
					AddReferencingProperty(PropertyIt->GetFName().ToString(), InReferencer->GetName(), InReferencedAsset->GetName());
				}
			}
		}
		else if (PropertyIt->IsA<FObjectProperty>())
		{
			UObject* Object;
			PropertyIt->GetValue_InContainer(InReferencer, &Object);
			if (Object == InReferencedAsset)
			{
				AddReferencingProperty(PropertyIt->GetDisplayNameText().ToString(), InReferencer->GetName(), InReferencedAsset->GetName(), FReferencingPropertyDescription::EAssetReferenceType::Value);
			}
		}
		// Other property (should handle Struct Property and fields as well)
		else
		{
			PropertyIt->ExportText_InContainer(0, PropertyExportString, InReferencer, InReferencer, InReferencer, PPF_IncludeTransient);

			if (!PropertyExportString.IsEmpty() && PropertyExportString.Contains(InReferencedAsset->GetPathName()))
			{
				AddReferencingProperty(PropertyIt->GetDisplayNameText().ToString(), InReferencer->GetName(), InReferencedAsset->GetName());
			}
		}
	}

	// The code above finds Assets when used as types (e.g. BP Enum, Struct or BPs) but not as values (e.g. a Static Mesh used as variable)
	// To find those, we serialize the ReferencingObject while looking for ReferencedObject referencing properties.

	class FArchiveReferencingProperties : public FArchiveUObject
	{
	public:
		FArchiveReferencingProperties(UObject* InReferencingObject, UObject* InReferencedObject, TArray<TTuple<FString, bool>>* OutReferencingProperties)
			: ReferencingProperties(OutReferencingProperties)
			, ReferencingObject(InReferencingObject)
			, ReferencedObject(InReferencedObject)
		{
			ArIsObjectReferenceCollector = true;
			ArIgnoreOuterRef = false;
			ArIgnoreArchetypeRef = true;
			ArIgnoreClassGeneratedByRef = true;
			ArIgnoreClassRef = true;

			SetShouldSkipCompilingAssets(false);
			ReferencingObjectPackage = ReferencingObject->GetPackage();
			ReferencingObject->Serialize(*this);
		}

		virtual FArchive& operator<<(UObject*& InSerializedObject) override
		{
			if (InSerializedObject)
			{
				if (InSerializedObject == ReferencedObject)
				{
					if (const FProperty* const Property = GetSerializedProperty())
					{
						if (const UObject* const PropertyOwner = Property->GetOwnerUObject())
						{
							// Make sure we are only showing properties which are part of the current package
							// This skips properties which mostly add no real meaningful information to the properties list.
							// Some might be nice to show, which will be taken care of in the future
							if (PropertyOwner->IsInPackage(ReferencingObjectPackage))
							{
								constexpr bool bIsIndirect = false;
								ReferencingProperties->AddUnique(TTuple<FString, bool>(Property->GetName(), bIsIndirect));
							}
						}
					}
				}
				else if (InSerializedObject->IsInPackage(ReferencingObjectPackage))
				{
					// Things like a Static Mesh referenced by a BP SM Component will generate what looks like a direct reference in the
					// graph. Let's gather those properties as well
					for (TFieldIterator<FObjectProperty> ObjectPropertyIt(InSerializedObject->GetClass()); ObjectPropertyIt;
						 ++ObjectPropertyIt)
					{
						if (UObject* ObjectReference =
								ObjectPropertyIt->GetObjectPropertyValue_InContainer(InSerializedObject))
						{
							if (ObjectReference == ReferencedObject)
							{
								FString PropertyName = InSerializedObject->GetFName().ToString();

								constexpr bool bIsIndirect = true;
								ReferencingProperties->AddUnique(TTuple<FString, bool>(PropertyName, bIsIndirect));
							}
						}
					}
				}

				if (InSerializedObject->IsInPackage(ReferencingObjectPackage))
				{
					bool bAlreadyExists;
					SerializedObjects.Add(InSerializedObject, &bAlreadyExists);

					if (!bAlreadyExists)
					{
						InSerializedObject->Serialize(*this);
					}
				}
			}

			return *this;
		}

	private:
		/** Stored pointer to array of objects we add object references to */
		TArray<TTuple<FString, bool>>* ReferencingProperties;

		/** Tracks the objects which have been serialized by this archive, to prevent recursion */
		TSet<UObject*> SerializedObjects;

		UObject* ReferencingObject;
		UObject* ReferencedObject;
		UPackage* ReferencingObjectPackage;
	};

	TArray<TTuple<FString, bool>> ReferencingPropertiesArray;
	FArchiveReferencingProperties Mapper(InReferencer, InReferencedAsset, &ReferencingPropertiesArray);
	for (const TTuple<FString, bool>& Property : ReferencingPropertiesArray)
	{
		const FString& PropertyName = Property.Get<0>();
		const bool bIsIndirect = Property.Get<1>();
		AddReferencingProperty(
				PropertyName, InReferencer->GetName(), InReferencedAsset->GetName(), FReferencingPropertyDescription::EAssetReferenceType::Value, bIsIndirect
			);
	}

	return ReferencingProperties;
}

UEdGraphNode_Reference* UEdGraph_ReferenceViewer::FindPath(const FAssetIdentifier& RootId, const FAssetIdentifier& TargetId)
{

	TargetIdentifier = TargetId;

	RemoveAllNodes();

	// Check for the target in the dependencies
	TMap<FAssetIdentifier, FReferenceNodeInfo> NewNodeInfos;
	TSet<FAssetIdentifier> Visited;
	FReferenceNodeInfo& RootNodeInfo = NewNodeInfos.FindOrAdd( RootId, FReferenceNodeInfo(RootId, false));
	if (TargetId.IsValid())
	{
		FindPath_Recursive(false, RootId, TargetId, NewNodeInfos, Visited);
	}
	GatherAssetData(NewNodeInfos);
	DependencyNodeInfos = NewNodeInfos;

	// Check for the target in the references
	Visited.Empty();
	TMap<FAssetIdentifier, FReferenceNodeInfo> NewRefNodeInfos;
	FReferenceNodeInfo& RootRefNodeInfo = NewRefNodeInfos.FindOrAdd( RootId, FReferenceNodeInfo(RootId, true));
	if (TargetId.IsValid())
	{
		FindPath_Recursive(true, RootId, TargetId, NewRefNodeInfos, Visited);
	}
	GatherAssetData(NewRefNodeInfos);
	ReferencerNodeInfos = NewRefNodeInfos;

	UEdGraphNode_Reference* NewRootNode = RefilterGraph();

	NotifyGraphChanged();

	return NewRootNode;
}

bool UEdGraph_ReferenceViewer::FindPath_Recursive(bool bInReferencers, const FAssetIdentifier& InAssetId, const FAssetIdentifier& TargetId, TMap<FAssetIdentifier, FReferenceNodeInfo>& InNodeInfos, TSet<FAssetIdentifier>& Visited )
{
	bool bFound = false;

	if (InAssetId == TargetId)
	{
		FReferenceNodeInfo& NewNodeInfo = InNodeInfos.FindOrAdd(InAssetId, FReferenceNodeInfo(InAssetId, bInReferencers));
		bFound = true;
	}

	// check if any decedents are the target and if any are found, add a node info for this asset as well 
	else 
	{
		Visited.Add(InAssetId);
		TMap<FAssetIdentifier, EDependencyPinCategory> ReferenceLinks;
		GetSortedLinks({InAssetId}, bInReferencers, GetReferenceSearchFlags(false), ReferenceLinks);

		for (const TPair<FAssetIdentifier, EDependencyPinCategory>& Pair : ReferenceLinks)
		{
			FAssetIdentifier ChildId = Pair.Key;
			if (!Visited.Contains(ChildId) && FindPath_Recursive(bInReferencers, ChildId, TargetId, InNodeInfos, Visited))
			{
				FReferenceNodeInfo& NewNodeInfo = InNodeInfos.FindOrAdd(InAssetId, FReferenceNodeInfo(InAssetId, bInReferencers));

				InNodeInfos[ChildId].Parents.AddUnique(InAssetId);
				InNodeInfos[InAssetId].Children.AddUnique(Pair);
				bFound = true;
			}
		}
	}

	return bFound;
}

UEdGraphNode_Reference* UEdGraph_ReferenceViewer::RefilterGraph()
{
	RemoveAllNodes();
	UEdGraphNode_Reference* RootNode = nullptr;

	bBreadthLimitReached = false;
	if (CurrentGraphRootIdentifiers.Num() > 0 && (!ReferencerNodeInfos.IsEmpty() || !DependencyNodeInfos.IsEmpty()))
	{
		FAssetIdentifier FirstGraphRootIdentifier = CurrentGraphRootIdentifiers[0];

		// Create the root node
		bool bRootIsDuplicated = false;

		for (const FAssetIdentifier& RootID : CurrentGraphRootIdentifiers)
		{
			bRootIsDuplicated |= (Settings->IsShowDependencies() && DependencyNodeInfos.Contains(RootID) && DependencyNodeInfos[RootID].IsADuplicate()) ||
				(Settings->IsShowReferencers() && ReferencerNodeInfos.Contains(RootID) && ReferencerNodeInfos[RootID].IsADuplicate());
		}
		for ( const FAssetIdentifier& RootID : CurrentGraphRootIdentifiers )
		{
			bRootIsDuplicated |= (Settings->IsShowDependencies() && DependencyNodeInfos.Contains(RootID) && DependencyNodeInfos[RootID].IsADuplicate()) ||
				(Settings->IsShowReferencers() && ReferencerNodeInfos.Contains(RootID) && ReferencerNodeInfos[RootID].IsADuplicate());
		}	

		const FReferenceNodeInfo& NodeInfo = Settings->IsShowReferencers() ? ReferencerNodeInfos[FirstGraphRootIdentifier] : DependencyNodeInfos[FirstGraphRootIdentifier];
		RootNode = CreateReferenceNode();
		RootNode->SetupReferenceNode(CurrentGraphRootOrigin, CurrentGraphRootIdentifiers, NodeInfo.AssetData, /*bInAllowThumbnail = */ !Settings->IsCompactMode(), /*bIsDuplicate*/ bRootIsDuplicated);
		RootNode->SetMakeCommentBubbleVisible(Settings->IsShowPath());

		if (Settings->IsShowReferencers())
		{
			RecursivelyFilterNodeInfos(FirstGraphRootIdentifier, ReferencerNodeInfos, 0, Settings->GetSearchReferencerDepthLimit());
			RecursivelyCreateNodes(true, FirstGraphRootIdentifier, CurrentGraphRootOrigin, FirstGraphRootIdentifier, RootNode, ReferencerNodeInfos, 0, Settings->GetSearchReferencerDepthLimit(), /*bIsRoot*/ true);
		}

		if (Settings->IsShowDependencies())
		{
			RecursivelyFilterNodeInfos(FirstGraphRootIdentifier, DependencyNodeInfos, 0, Settings->GetSearchDependencyDepthLimit());
			RecursivelyCreateNodes(false, FirstGraphRootIdentifier, CurrentGraphRootOrigin, FirstGraphRootIdentifier, RootNode, DependencyNodeInfos, 0, Settings->GetSearchDependencyDepthLimit(), /*bIsRoot*/ true);
		}
	}

	NotifyGraphChanged();
	return RootNode;
}

void UEdGraph_ReferenceViewer::RecursivelyFilterNodeInfos(const FAssetIdentifier& InAssetId, TMap<FAssetIdentifier, FReferenceNodeInfo>& InNodeInfos, int32 InCurrentDepth, int32 InMaxDepth)
{
	// Filters and Re-provisions the NodeInfo counts 
	int32 NewProvisionSize = 0;

	int32 Breadth = 0;

	FReferenceNodeInfo& NodeInfo = InNodeInfos[InAssetId];

	int32 CurrentDepth = InCurrentDepth;
	int32 CurrentMaxDepth = InMaxDepth;
	if (NodeInfo.IsRedirector())
	{
		// We don't count depth for redirectors
		CurrentDepth = 0;
		CurrentMaxDepth = InMaxDepth - InCurrentDepth + 1;
	}
	
	NodeInfo.OverflowCount = 0;
	if (!ExceedsMaxSearchDepth(CurrentDepth, CurrentMaxDepth))
	{
		for (const TPair<FAssetIdentifier, EDependencyPinCategory>& Pair : NodeInfo.Children)
		{
			FAssetIdentifier ChildId = Pair.Key;
			const FReferenceNodeInfo& ChildNodeInfo = InNodeInfos[ChildId];

			int32 ChildProvSize = 0;
			if (ChildNodeInfo.IsFirstParent(InAssetId))
			{
				RecursivelyFilterNodeInfos(ChildId, InNodeInfos, CurrentDepth + 1, CurrentMaxDepth);
				ChildProvSize = ChildNodeInfo.ProvisionSize(InAssetId);
			}
			else if (Settings->GetFindPathEnabled())
			{
				ChildProvSize = 1;
			}
			else if (ChildNodeInfo.PassedFilters && Settings->IsShowDuplicates())
			{
				ChildProvSize = 1;
			}

			if (ChildProvSize > 0)
			{
				if (!ExceedsMaxSearchBreadth(Breadth) || NodeInfo.bExpandAllChildren)
				{
					NewProvisionSize += ChildProvSize;
					Breadth++;
				}

				else
				{
					NodeInfo.OverflowCount++;
					Breadth++;
				}
			}
		}
	}

	// Account for an overflow node if necessary
	if (NodeInfo.OverflowCount > 0)
	{
		NewProvisionSize++;
		bBreadthLimitReached = true;
	}

	bool PassedAssetTypeFilter = FilterCollection && Settings->GetFiltersEnabled() ? FilterCollection->PassesAllFilters(NodeInfo) : true;
	bool PassedSearchTextFilter = DoesAssetPassSearchTextFilter(InAssetId, NodeInfo.AssetData);

	// Don't apply filters in Find Path Mode. Otherwise, check the type and search filters, and also don't include any assets in the central selection (where InCurrentDepth == 0)
	bool PassedAllFilters = Settings->GetFindPathEnabled() || (PassedAssetTypeFilter && PassedSearchTextFilter && (CurrentDepth == 0 || !CurrentGraphRootIdentifiers.Contains(InAssetId)));

	NodeInfo.ChildProvisionSize = NewProvisionSize > 0 ? NewProvisionSize : (PassedAllFilters ? 1 : 0);
	NodeInfo.PassedFilters = PassedAllFilters;
}

void UEdGraph_ReferenceViewer::GetSortedLinks(const TArray<FAssetIdentifier>& Identifiers, bool bReferencers, const FAssetManagerDependencyQuery& Query, TMap<FAssetIdentifier, EDependencyPinCategory>& OutLinks) const
{
	using namespace UE::AssetRegistry;
	auto CategoryOrder = [](EDependencyCategory InCategory)
	{
		switch (InCategory)
		{
			case EDependencyCategory::Package: 
			{
				return 0;
			}
			case EDependencyCategory::Manage:
			{
				return 1;
			}
			case EDependencyCategory::SearchableName: 
			{
				return 2;
			}
			default: 
			{
				check(false);
				return 3;
			}
		}
	};
	auto IsHard = [](EDependencyProperty Properties)
	{
		return static_cast<bool>(((Properties & EDependencyProperty::Hard) != EDependencyProperty::None) | ((Properties & EDependencyProperty::Direct) != EDependencyProperty::None));
	};

	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	TArray<FAssetDependency> LinksToAsset;
	for (const FAssetIdentifier& AssetId : Identifiers)
	{
		LinksToAsset.Reset();
		if (bReferencers)
		{
			AssetRegistry.GetReferencers(AssetId, LinksToAsset, Query.Categories, Query.Flags);

			if (!Settings->IsShowExternalReferencers())
			{
				TSet<FName> PackageNames;
				for (const FAssetDependency& LinkToAsset : LinksToAsset)
				{
					if (!LinkToAsset.AssetId.IsValue() && !LinkToAsset.AssetId.PackageName.IsNone())
					{
						PackageNames.Add(LinkToAsset.AssetId.PackageName);
					}
				}

				TMap<FName, FAssetData> PackagesToAssetDataMap;
				UE::AssetRegistry::GetAssetForPackages(PackageNames.Array(), PackagesToAssetDataMap);

				TSet<FName> OuterPathNames;
				for (int32 LinksToAssetIndex = 0; LinksToAssetIndex < LinksToAsset.Num(); LinksToAssetIndex++)
				{
					FAssetDependency& AssetDependency = LinksToAsset[LinksToAssetIndex];				
					if (FAssetData* AssetData = PackagesToAssetDataMap.Find(AssetDependency.AssetId.PackageName))
					{
						if (FName OuterPathName = AssetData->GetOptionalOuterPathName(); !OuterPathName.IsNone())
						{
							if (!OuterPathNames.Contains(OuterPathName))
							{
								FAssetDependency OuterDependency;
								OuterDependency.AssetId = FAssetIdentifier(*FSoftObjectPath(OuterPathName.ToString()).GetLongPackageName());
								OuterDependency.Category = AssetDependency.Category;
								OuterDependency.Properties = AssetDependency.Properties;
								LinksToAsset.Add(OuterDependency);

								OuterPathNames.Add(OuterPathName);
							}

							LinksToAsset.RemoveAtSwap(LinksToAssetIndex--);
						}
					}
				}
			}
		}
		else
		{
			AssetRegistry.GetDependencies(AssetId, LinksToAsset, Query.Categories, Query.Flags);
		}

		// Sort the links from most important kind of link to least important kind of link, so that if we can't display them all in an ExceedsMaxSearchBreadth test, we
		// show the most important links.
		Algo::Sort(LinksToAsset, [&CategoryOrder, &IsHard](const FAssetDependency& A, const FAssetDependency& B)
			{
				if (A.Category != B.Category)
				{
					return CategoryOrder(A.Category) < CategoryOrder(B.Category);
				}
				if (A.Properties != B.Properties)
				{
					bool bAIsHard = IsHard(A.Properties);
					bool bBIsHard = IsHard(B.Properties);
					if (bAIsHard != bBIsHard)
					{
						return bAIsHard;
					}
				}
				return A.AssetId.PackageName.LexicalLess(B.AssetId.PackageName);
			});
		for (FAssetDependency LinkToAsset : LinksToAsset)
		{
			EDependencyPinCategory& Category = OutLinks.FindOrAdd(LinkToAsset.AssetId, EDependencyPinCategory::LinkEndActive);
			bool bIsHard = IsHard(LinkToAsset.Properties);
			bool bIsUsedInGame = (LinkToAsset.Category != EDependencyCategory::Package) || ((LinkToAsset.Properties & EDependencyProperty::Game) != EDependencyProperty::None);
			Category |= EDependencyPinCategory::LinkEndActive;
			Category |= bIsHard ? EDependencyPinCategory::LinkTypeHard : EDependencyPinCategory::LinkTypeNone;
			Category |= bIsUsedInGame ? EDependencyPinCategory::LinkTypeUsedInGame : EDependencyPinCategory::LinkTypeNone;
		}
	}

	// Check filters and Filter for our registry source
	TArray<FAssetIdentifier> ReferenceIds;
	OutLinks.GenerateKeyArray(ReferenceIds);
	IAssetManagerEditorModule::Get().FilterAssetIdentifiersForCurrentRegistrySource(ReferenceIds, GetReferenceSearchFlags(false), !bReferencers);


	// The following for loop might take a long time for certain assets/classes - show a progress bar dialog
	FScopedSlowTask LinksCleanupTask(OutLinks.Num(), LOCTEXT("LinksCleanupTask", "Processing Reference Viewer graph links"));

	// Used to discriminate lightweight vs. heavy load set of links
	const bool bIsSlowTask = OutLinks.Num() > 500;
	if (bIsSlowTask)
	{
		LinksCleanupTask.MakeDialog();
	}

	for (TMap<FAssetIdentifier, EDependencyPinCategory>::TIterator It(OutLinks); It; ++It)
	{
		if (bIsSlowTask)
		{
			LinksCleanupTask.EnterProgressFrame();
		}

		if (!IsPackageIdentifierPassingFilter(It.Key()))
		{
			It.RemoveCurrent();
		}

		else if (!ReferenceIds.Contains(It.Key()))
		{
			It.RemoveCurrent();
		}

		// Collection Filter
		else if (ShouldFilterByCollection() && It.Key().IsPackage() && !CurrentCollectionPackages.Contains(It.Key().PackageName))
		{
			It.RemoveCurrent();
		}

		else if (!IsPackageIdentifierPassingPluginFilter(It.Key()))
		{
			It.RemoveCurrent();
		}
	}
}

bool UEdGraph_ReferenceViewer::IsPackageIdentifierPassingFilter(const FAssetIdentifier& InAssetIdentifier) const
{
	if (!InAssetIdentifier.IsValue())
	{
		if (!Settings->IsShowCodePackages() && InAssetIdentifier.PackageName.ToString().StartsWith(TEXT("/Script")))
		{
			return false;
		}

	}

	return true;
}

bool UEdGraph_ReferenceViewer::IsPackageIdentifierPassingPluginFilter(const FAssetIdentifier& InAssetIdentifier) const
{
	if (!ShouldFilterByPlugin())
	{
		return true;
	}
	
	if (!InAssetIdentifier.IsPackage())
	{
		return true;
	}
	
	const FString AssetPath = InAssetIdentifier.PackageName.ToString();

	for (const FName& PluginName : CurrentPluginFilter)
	{
		if (AssetPath.StartsWith("/" + PluginName.ToString()))
		{
			return true;
		}
	}

	return false;
}

bool UEdGraph_ReferenceViewer::DoesAssetPassSearchTextFilter(const FAssetIdentifier& InAssetIdentifier, const FAssetData& InAssetData) const
{
	if (Settings->IsShowFilteredPackagesOnly() && DoesAssetPassSearchFilterCallback.IsSet() && !DoesAssetPassSearchFilterCallback(InAssetIdentifier, InAssetData))
	{
		return false;
	}

	return true;
}

void UEdGraph_ReferenceViewer::GetUnfilteredGraphPluginNamesRecursive(bool bReferencers, const FAssetIdentifier& InAssetIdentifier, int32 InCurrentDepth, int32 InMaxDepth, const FAssetManagerDependencyQuery& Query, TSet<FAssetIdentifier>& OutAssetIdentifiers)
{
	if (ExceedsMaxSearchDepth(InCurrentDepth, InMaxDepth))
	{
		return;
	}
	
	IAssetRegistry& AssetRegistry = FModuleManager::LoadModuleChecked<FAssetRegistryModule>("AssetRegistry").Get();
	TArray<FAssetDependency> LinksToAsset;
	if (bReferencers)
	{
		AssetRegistry.GetReferencers(InAssetIdentifier, LinksToAsset);
	}
	else
	{
		AssetRegistry.GetDependencies(InAssetIdentifier, LinksToAsset);
	}
	
	for (const FAssetDependency& Link : LinksToAsset)
	{
		// Avoid loops by skipping assets we've already visited.
		if (OutAssetIdentifiers.Contains(Link.AssetId))
		{
			continue;;
		}

		// Don't add assets that will be hidden by Reference Viewer settings the user cannot change.
		if (!IsPackageIdentifierPassingFilter(Link.AssetId))
		{
			continue;
		}

		OutAssetIdentifiers.Add(Link.AssetId);

		GetUnfilteredGraphPluginNamesRecursive(bReferencers, Link.AssetId, InCurrentDepth + 1, InMaxDepth, Query, OutAssetIdentifiers);
	}
}

void UEdGraph_ReferenceViewer::GetUnfilteredGraphPluginNames(TArray<FAssetIdentifier> RootIdentifiers, TArray<FName>& OutPluginNames)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UEdGraph_ReferenceViewer::GetUnfilteredGraphPluginNames);
	
	const FAssetManagerDependencyQuery Query = GetReferenceSearchFlags(false);
	
	TSet<FAssetIdentifier> AssetIdentifiers;
	for (const FAssetIdentifier& RootIdentifier : RootIdentifiers)
	{
		TSet<FAssetIdentifier> AssetReferencerIdentifiers;
		GetUnfilteredGraphPluginNamesRecursive(true, RootIdentifier, 0, Settings->GetSearchReferencerDepthLimit(), Query, AssetReferencerIdentifiers);
		AssetIdentifiers.Append(AssetReferencerIdentifiers);

		TSet<FAssetIdentifier> AssetDependencyIdentifiers;
		GetUnfilteredGraphPluginNamesRecursive(false, RootIdentifier, 0, Settings->GetSearchDependencyDepthLimit(), Query, AssetDependencyIdentifiers);
		AssetIdentifiers.Append(AssetDependencyIdentifiers);
	}

	for (const FAssetIdentifier& AssetIdentifier : AssetIdentifiers)
	{
		if (!AssetIdentifier.IsPackage())
		{
			continue;
		}

		FString FirstPathSegment;
		{
			FString AssetPath = AssetIdentifier.PackageName.ToString();
			
			// Chop of any leading slashes.
			while (AssetPath.StartsWith("/"))
			{
				AssetPath = AssetPath.Mid(1);
			}

			const int32 SecondSlash = AssetPath.Find("/");
			if (SecondSlash != INDEX_NONE)
			{
				AssetPath = AssetPath.Left(SecondSlash);
			}
			
			FirstPathSegment = AssetPath;
		}

		OutPluginNames.AddUnique(FName(FirstPathSegment));
	}
}

void
UEdGraph_ReferenceViewer::RecursivelyPopulateNodeInfos(bool bInReferencers, const TArray<FAssetIdentifier>& Identifiers, TMap<FAssetIdentifier, FReferenceNodeInfo>& InNodeInfos, int32 InCurrentDepth, int32 InMaxDepth)
{
	check(Identifiers.Num() > 0);
	int32 ProvisionSize = 0;
	const FAssetIdentifier& InAssetId = Identifiers[0];

	bool bIsRedirector = false;

	// Check if this node is actually a redirector
	TMap<FName, FAssetData> PackageToAssetDataMap;
	UE::AssetRegistry::GetAssetForPackages({InAssetId.PackageName}, PackageToAssetDataMap);

	FAssetData* AssetData = PackageToAssetDataMap.Find(InAssetId.PackageName);
	if (!bInReferencers && AssetData && AssetData->IsRedirector())
	{
		bIsRedirector = true;

		IAssetRegistry* AssetRegistry = IAssetRegistry::Get();
		check(AssetRegistry);

		// We are dealing with a redirector. Let's manually retrieve its Destination Object, and set up its set of nodes explicitly
		FAssetData DestinationObjectAssetData = *AssetData;
		TSet<FString> AllVisitedObjectPaths;
		while (DestinationObjectAssetData.IsRedirector())
		{
			FString DestinationObject = FPackageName::ExportTextPathToObjectPath(DestinationObjectAssetData.GetTagValueRef<FString>(GET_MEMBER_NAME_CHECKED(UObjectRedirector, DestinationObject)));
			if (DestinationObject.IsEmpty() || AllVisitedObjectPaths.Contains(DestinationObject))
			{
				// No destination or a loop
				DestinationObjectAssetData = FAssetData();
				break;
			}

			AllVisitedObjectPaths.Add(DestinationObject);
			DestinationObjectAssetData = AssetRegistry->GetAssetByObjectPath(DestinationObject, true);
		}

		if (DestinationObjectAssetData.IsValid())
		{
			const FName DestinationPackageName = DestinationObjectAssetData.PackageName;
			const FAssetIdentifier DestinationAssetId = FAssetIdentifier::FromString(DestinationPackageName.ToString());

			FReferenceNodeInfo& DestinationReferenceNodeInfo = InNodeInfos.FindOrAdd(DestinationAssetId, FReferenceNodeInfo(DestinationAssetId, bInReferencers));

			// The Destination Node parent is the Redirector one
			DestinationReferenceNodeInfo.Parents.Emplace(InAssetId);

			// Remove Children from Redirector Node, and just add the Destination Node
			InNodeInfos[InAssetId].Children.Empty();
			InNodeInfos[InAssetId].Children.Emplace(DestinationAssetId, EDependencyPinCategory::LinkTypeHard);
			InNodeInfos[InAssetId].bIsRedirector = true;

			// Populate Info, without increasing current depth - we ignore the Redirector
			RecursivelyPopulateNodeInfos(bInReferencers, { DestinationAssetId }, InNodeInfos, 0, InMaxDepth - InCurrentDepth);
		}
	}

	if (!bIsRedirector && !ExceedsMaxSearchDepth(InCurrentDepth, InMaxDepth))
	{
		TMap<FAssetIdentifier, EDependencyPinCategory> ReferenceLinks;
		GetSortedLinks(Identifiers, bInReferencers, GetReferenceSearchFlags(false), ReferenceLinks);

		// If already available, store Asset Data in Reference Node info
		if (AssetData)
		{
			InNodeInfos[InAssetId].AssetData = *AssetData;
		}

		InNodeInfos[InAssetId].Children.Reserve(ReferenceLinks.Num());
		for (const TPair<FAssetIdentifier, EDependencyPinCategory>& Pair : ReferenceLinks)
		{
			FAssetIdentifier ChildId = Pair.Key;
			if (!InNodeInfos.Contains(ChildId))
			{
				FReferenceNodeInfo& NewNodeInfo = InNodeInfos.FindOrAdd(ChildId, FReferenceNodeInfo(ChildId, bInReferencers));
				InNodeInfos[ChildId].Parents.Emplace(InAssetId);
				InNodeInfos[InAssetId].Children.Emplace(Pair);

				RecursivelyPopulateNodeInfos(bInReferencers, { ChildId }, InNodeInfos, InCurrentDepth + 1, InMaxDepth);
				ProvisionSize += InNodeInfos[ChildId].ProvisionSize(InAssetId);
			}

			else if (!InNodeInfos[ChildId].Parents.Contains(InAssetId))
			{
				InNodeInfos[ChildId].Parents.Emplace(InAssetId);
				InNodeInfos[InAssetId].Children.Emplace(Pair);
				ProvisionSize += 1;
			}
		}
	}

	// Account for an overflow node if necessary
	if (InNodeInfos[InAssetId].OverflowCount > 0)
	{
		ProvisionSize++;
	}

	InNodeInfos[InAssetId].ChildProvisionSize = ProvisionSize > 0 ? ProvisionSize : 1;
}

void UEdGraph_ReferenceViewer::GatherAssetData(TMap<FAssetIdentifier, FReferenceNodeInfo>& InNodeInfos)
{
	// Grab the list of packages
	TSet<FName> PackageNames;
	for (TPair<FAssetIdentifier, FReferenceNodeInfo>&  InfoPair : InNodeInfos)
	{
		FAssetIdentifier& AssetId = InfoPair.Key;
		if (!AssetId.IsValue() && !AssetId.PackageName.IsNone())
		{
			PackageNames.Add(AssetId.PackageName);
		}
	}

	// Retrieve the AssetData from the Registry
	TMap<FName, FAssetData> PackagesToAssetDataMap;
	UE::AssetRegistry::GetAssetForPackages(PackageNames.Array(), PackagesToAssetDataMap);


	// Populate the AssetData back into the NodeInfos
	for (TPair<FAssetIdentifier, FReferenceNodeInfo>&  InfoPair : InNodeInfos)
	{
		InfoPair.Value.AssetData = PackagesToAssetDataMap.FindRef(InfoPair.Key.PackageName);
	}
}

UEdGraphNode_Reference* UEdGraph_ReferenceViewer::RecursivelyCreateNodes(bool bInReferencers, const FAssetIdentifier& InAssetId, const FIntPoint& InNodeLoc, const FAssetIdentifier& InParentId, UEdGraphNode_Reference* InParentNode, TMap<FAssetIdentifier, FReferenceNodeInfo>& InNodeInfos, int32 InCurrentDepth, int32 InMaxDepth, bool bIsRoot)
{
	check(InNodeInfos.Contains(InAssetId));

	const FReferenceNodeInfo& NodeInfo = InNodeInfos[InAssetId];
	int32 NodeProvSize = 1;

	int32 CurrentDepth = InCurrentDepth;
	int32 CurrentMaxDepth = InMaxDepth;

	UEdGraphNode_Reference* NewNode = nullptr;
	if (bIsRoot)
	{
		NewNode = InParentNode;
		NodeProvSize = NodeInfo.ProvisionSize(FAssetIdentifier(NAME_None));
	}
	else
	{
		NewNode = CreateReferenceNode();
		NewNode->SetupReferenceNode(InNodeLoc, {InAssetId}, NodeInfo.AssetData, /*bInAllowThumbnail*/ !Settings->IsCompactMode() && NodeInfo.PassedFilters, /*bIsADuplicate*/ NodeInfo.Parents.Num() > 1);
		NewNode->SetMakeCommentBubbleVisible(Settings->IsShowPath());
		NewNode->SetIsFiltered(!NodeInfo.PassedFilters);
		NodeProvSize = NodeInfo.ProvisionSize(InParentId);
	}

	FIntPoint ChildLoc = InNodeLoc;
	if (NodeInfo.IsRedirector())
	{
		// We don't count depth for redirectors
		CurrentDepth = 0;
		CurrentMaxDepth = InMaxDepth - InCurrentDepth + 1;
	}
	
	bool bIsFirstOccurance = bIsRoot || NodeInfo.IsFirstParent(InParentId);
	if (!ExceedsMaxSearchDepth(CurrentDepth, InMaxDepth) && bIsFirstOccurance) // Only expand the first parent
	{

		// position the children nodes
		const int32 ColumnWidth = Settings->IsCompactMode() ? 500 : 800;
		ChildLoc.X += bInReferencers ? -ColumnWidth : ColumnWidth;

		int32 NodeSizeY = Settings->IsCompactMode() ? 100 : 200;
		NodeSizeY += Settings->IsShowPath() ? 40 : 0;

		ChildLoc.Y -= (NodeProvSize - 1) * NodeSizeY * 0.5 ;

		int32 Breadth = 0;
		int32 ChildIdx = 0;
		for (; ChildIdx < InNodeInfos[InAssetId].Children.Num(); ChildIdx++)
		{
			const TPair<FAssetIdentifier, EDependencyPinCategory>& Pair = InNodeInfos[InAssetId].Children[ChildIdx];
			if (ExceedsMaxSearchBreadth(Breadth) && !InNodeInfos[InAssetId].bExpandAllChildren)
			{
				break;
			}

			FAssetIdentifier ChildId = Pair.Key;
			int32 ChildProvSize = 0;
			if (InNodeInfos[ChildId].IsFirstParent(InAssetId))
			{
				ChildProvSize = InNodeInfos[ChildId].ProvisionSize(InAssetId);
			}
			else if (Settings->GetFindPathEnabled())
			{
				ChildProvSize = 1;
			}
			else if (InNodeInfos[ChildId].PassedFilters && Settings->IsShowDuplicates())
			{
				ChildProvSize = 1;
			}

			// The provision size will always be at least 1 if it should be shown, factoring in filters, breadth, duplicates, etc.
			if (ChildProvSize > 0)
			{
				ChildLoc.Y += (ChildProvSize - 1) * NodeSizeY * 0.5;

				UEdGraphNode_Reference* ChildNode = RecursivelyCreateNodes(bInReferencers, ChildId, ChildLoc, InAssetId, NewNode, InNodeInfos, CurrentDepth + 1, CurrentMaxDepth);

				if (bInReferencers)
				{
					ChildNode->GetDependencyPin()->PinType.PinCategory = ::GetName(Pair.Value);
					NewNode->AddReferencer( ChildNode );
				}
				else
				{
					ChildNode->GetReferencerPin()->PinType.PinCategory = ::GetName(Pair.Value);
					ChildNode->AddReferencer( NewNode );
				}

				ChildLoc.Y += NodeSizeY * (ChildProvSize + 1) * 0.5;
				Breadth ++;
			}
		}

		// There were more references than allowed to be displayed. Make a collapsed node.
		if (NodeInfo.OverflowCount > 0)
		{
			UEdGraphNode_Reference* OverflowNode = nullptr;
			FIntPoint RefNodeLoc;
			RefNodeLoc.X = ChildLoc.X;
			RefNodeLoc.Y = ChildLoc.Y;

			// Overflow count is 1: instead of collapsing a single node, we can directly display it
			if (NodeInfo.OverflowCount == 1)
			{
				// Reaching the overflowing node
				if (NodeInfo.Children.IsValidIndex(Breadth))
				{
					const TPair<FAssetIdentifier, EDependencyPinCategory>& OverflowNodePair = NodeInfo.Children[Breadth];

					const FAssetIdentifier& OverflowNodeAssetId = OverflowNodePair.Key;
					OverflowNode = RecursivelyCreateNodes(bInReferencers, OverflowNodeAssetId, ChildLoc, NodeInfo.AssetId, NewNode, InNodeInfos, CurrentDepth + 1, CurrentMaxDepth);

					// Make sure to keep track of pin category (e.g. soft vs hard ref)
					const TPair<FAssetIdentifier, EDependencyPinCategory>& Pair = InNodeInfos[InAssetId].Children[Breadth];
					FName PinCategory = ::GetName(Pair.Value);

					if (bInReferencers)
					{
						OverflowNode->GetDependencyPin()->PinType.PinCategory = PinCategory;
					}
					else
					{
						OverflowNode->GetReferencerPin()->PinType.PinCategory = PinCategory;
					}
				}
			}

			// OverflowNode is not valid. Either NodeInfo.OverflowCount is > 1, or single node creation failed.
			// Let's create a collapsed node.
			if (!OverflowNode)
			{
				if (UEdGraphNode_Reference* CollapsedNode = CreateReferenceNode())
				{
					TArray<FAssetIdentifier> CollapsedNodeIdentifiers;
					for (; ChildIdx < InNodeInfos[InAssetId].Children.Num(); ChildIdx++)
					{
						const TPair<FAssetIdentifier, EDependencyPinCategory>& Pair = InNodeInfos[InAssetId].Children[ChildIdx];
						CollapsedNodeIdentifiers.Add(Pair.Key);
					}

					CollapsedNode->SetReferenceNodeCollapsed(RefNodeLoc, NodeInfo.OverflowCount, CollapsedNodeIdentifiers);
					OverflowNode = CollapsedNode;
				}
			}

			if (ensure(OverflowNode))
			{
				OverflowNode->SetAllowThumbnail(!Settings->IsCompactMode());

				if (bInReferencers)
				{
					NewNode->AddReferencer(OverflowNode);
				}
				else
				{
					OverflowNode->AddReferencer(NewNode);
				}
			}
		}
	}

	return NewNode;
}

void UEdGraph_ReferenceViewer::ExpandNode(bool bReferencers, const FAssetIdentifier& InAssetIdentifier)
{
	if (!bReferencers && DependencyNodeInfos.Contains(InAssetIdentifier))
	{
		DependencyNodeInfos[InAssetIdentifier].bExpandAllChildren = true;
		RefilterGraph();
	}

	else if (bReferencers && ReferencerNodeInfos.Contains(InAssetIdentifier))
	{
		ReferencerNodeInfos[InAssetIdentifier].bExpandAllChildren = true;
		RefilterGraph();
	}
}

const TSharedPtr<FAssetThumbnailPool>& UEdGraph_ReferenceViewer::GetAssetThumbnailPool() const
{
	return AssetThumbnailPool;
}

bool UEdGraph_ReferenceViewer::ExceedsMaxSearchDepth(int32 Depth, int32 MaxDepth) const
{

	const bool bIsWithinDepthLimits = (MaxDepth > 0 && Depth < MaxDepth);
	// the FindPath feature is not depth limited
 	if (Settings->GetFindPathEnabled())
 	{
 		return false;
 	}
 	else if (Settings->IsSearchDepthLimited() && !bIsWithinDepthLimits)
 	{
 		return true;
 	}

 	return false;
}

bool UEdGraph_ReferenceViewer::ExceedsMaxSearchBreadth(int32 Breadth) const
{
	// the FindPath feature is not breadth limited
	if (Settings->GetFindPathEnabled())
	{
		return false;
	}

	// ExceedsMaxSearchBreadth requires greater or equal than because the Breadth is 1-based indexed
	return Breadth >= Settings->GetSearchBreadthLimit();
}

UEdGraphNode_Reference* UEdGraph_ReferenceViewer::CreateReferenceNode()
{
	const bool bSelectNewNode = false;
	return Cast<UEdGraphNode_Reference>(CreateNode(UEdGraphNode_Reference::StaticClass(), bSelectNewNode));
}

UEdGraphNode_ReferencedProperties* UEdGraph_ReferenceViewer::CreateReferencedPropertiesNode(
	const TArray<FReferencingPropertyDescription>& InPropertiesDescriptionArray,
	const TObjectPtr<UEdGraphNode_Reference>& InReferencingNode,
	const TObjectPtr<UEdGraphNode_Reference>& InReferencedNode
)
{
	uint32 NodesPairHash = GetTypeHash(InReferencingNode) ^ GetTypeHash(InReferencedNode);

	UEdGraphNode_ReferencedProperties* PropertiesNode = nullptr;

	if (ReferencedPropertiesNodes.Contains(NodesPairHash))
	{
		if (TWeakObjectPtr<UEdGraphNode_ReferencedProperties>* PropertiesNodePtr =
			ReferencedPropertiesNodes.Find(NodesPairHash))
		{
			if (PropertiesNodePtr->IsValid())
			{
				PropertiesNode = PropertiesNodePtr->Get();
			}
		}
	}
	else
	{
		constexpr bool bSelectNewNode = false;
		PropertiesNode = Cast<UEdGraphNode_ReferencedProperties>(
			CreateNode(UEdGraphNode_ReferencedProperties::StaticClass(), bSelectNewNode)
		);

		ReferencedPropertiesNodes.Emplace(NodesPairHash, PropertiesNode);
	}

	if (PropertiesNode)
	{
		PropertiesNode->SetupReferencedPropertiesNode(InPropertiesDescriptionArray, InReferencingNode, InReferencedNode);
	}

	return PropertiesNode;
}

void UEdGraph_ReferenceViewer::RemoveAllNodes()
{
	TArray<UEdGraphNode*> NodesToRemove = Nodes;
	for (int32 NodeIndex = 0; NodeIndex < NodesToRemove.Num(); ++NodeIndex)
	{
		RemoveNode(NodesToRemove[NodeIndex]);
	}
}

bool UEdGraph_ReferenceViewer::ShouldFilterByCollection() const
{
	return Settings->GetEnableCollectionFilter() && CurrentCollectionFilterContainer.IsValid() && CurrentCollectionFilterName != NAME_None;
}

bool UEdGraph_ReferenceViewer::ShouldFilterByPlugin() const
{
	return Settings->GetEnablePluginFilter() && CurrentPluginFilter.Num() > 0;
}

#undef LOCTEXT_NAMESPACE
