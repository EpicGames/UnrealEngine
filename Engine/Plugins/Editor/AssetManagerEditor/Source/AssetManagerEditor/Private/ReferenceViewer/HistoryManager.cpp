// Copyright Epic Games, Inc. All Rights Reserved.

#include "HistoryManager.h"
#include "AssetRegistry/AssetIdentifier.h"

#define LOCTEXT_NAMESPACE "ReferenceViewer"

FReferenceViewerHistoryManager::FReferenceViewerHistoryManager()
{
	CurrentHistoryIndex = 0;
	MaxHistoryEntries = 300;
}

void FReferenceViewerHistoryManager::SetOnApplyHistoryData(const FOnApplyHistoryData& InOnApplyHistoryData)
{
	OnApplyHistoryData = InOnApplyHistoryData;
}

void FReferenceViewerHistoryManager::SetOnUpdateHistoryData(const FOnUpdateHistoryData& InOnUpdateHistoryData)
{
	OnUpdateHistoryData = InOnUpdateHistoryData;
}

bool FReferenceViewerHistoryManager::GoBack()
{
	if ( CanGoBack() )
	{
		// Update the current history data
		UpdateCurrentHistoryData();

		// if its possible to go back, decrement the index we are at
		--CurrentHistoryIndex;

		// Update the owner
		ApplyCurrentHistoryData();

		return true;
	}

	return false;
}

bool FReferenceViewerHistoryManager::GoForward()
{
	if ( CanGoForward() )
	{
		// Update the current history data
		UpdateCurrentHistoryData();

		// if its possible to go forward, increment the index we are at
		++CurrentHistoryIndex;

		// Update the owner
		ApplyCurrentHistoryData();

		return true;
	}

	return false;
}

void FReferenceViewerHistoryManager::AddHistoryData()
{
	if (HistoryData.Num() == 0)
	{
		// History added to the beginning
		HistoryData.Add(FReferenceViewerHistoryData());
		CurrentHistoryIndex = 0;
	}
	else if (CurrentHistoryIndex == HistoryData.Num() - 1)
	{
		// History added to the end
		if (HistoryData.Num() == MaxHistoryEntries)
		{
			// If max history entries has been reached
			// remove the oldest history
			HistoryData.RemoveAt(0);
		}
		HistoryData.Add(FReferenceViewerHistoryData());
		// Current history index is the last index in the list
		CurrentHistoryIndex = HistoryData.Num() - 1;
	}
	else
	{
		// History added to the middle
		// clear out all history after the current history index.
		HistoryData.RemoveAt(CurrentHistoryIndex + 1, HistoryData.Num() - (CurrentHistoryIndex + 1));
		HistoryData.Add(FReferenceViewerHistoryData());
		// Current history index is the last index in the list
		CurrentHistoryIndex = HistoryData.Num() - 1;
	}

	// Update the current history data
	UpdateCurrentHistoryData();
}

void FReferenceViewerHistoryManager::UpdateHistoryData()
{
	// Update the current history data
	UpdateCurrentHistoryData();
}

bool FReferenceViewerHistoryManager::CanGoForward() const
{
	// User can go forward if there are items in the history data list, 
	// and the current history index isn't the last index in the list
	return HistoryData.Num() > 0 && CurrentHistoryIndex < HistoryData.Num() - 1;
}

bool FReferenceViewerHistoryManager::CanGoBack() const
{
	// User can go back if there are items in the history data list,
	// and the current history index isn't the first index in the list
	return HistoryData.Num() > 0 && CurrentHistoryIndex > 0;
}

const FReferenceViewerHistoryData* FReferenceViewerHistoryManager::GetBackHistoryData() const
{
	if ( CanGoBack() )
	{
		return &HistoryData[CurrentHistoryIndex - 1];
	}
	return nullptr;
}

const FReferenceViewerHistoryData* FReferenceViewerHistoryManager::GetForwardHistoryData() const
{
	if ( CanGoForward() )
	{
		return &HistoryData[CurrentHistoryIndex + 1];
	}
	return nullptr;
}

void FReferenceViewerHistoryManager::ApplyCurrentHistoryData()
{
	if ( CurrentHistoryIndex >= 0 && CurrentHistoryIndex < HistoryData.Num())
	{
		OnApplyHistoryData.ExecuteIfBound( HistoryData[CurrentHistoryIndex] );
	}
}

void FReferenceViewerHistoryManager::UpdateCurrentHistoryData()
{
	if ( CurrentHistoryIndex >= 0 && CurrentHistoryIndex < HistoryData.Num())
	{
		OnUpdateHistoryData.ExecuteIfBound( HistoryData[CurrentHistoryIndex] );
	}
}

void FReferenceViewerHistoryManager::ExecuteJumpToHistory(int32 HistoryIndex)
{
	if (HistoryIndex >= 0 && HistoryIndex < HistoryData.Num())
	{
		// if the history index is valid, set the current history index to the history index requested by the user
		CurrentHistoryIndex = HistoryIndex;

		// Update the owner
		ApplyCurrentHistoryData();
	}
}

#undef LOCTEXT_NAMESPACE
