// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceViewerFilters.h"

#include "ISourceControlModule.h"
#include "ISourceControlProvider.h"
#include "ReferenceViewer/EdGraph_ReferenceViewer.h"
#include "SourceControlOperations.h"

#define LOCTEXT_NAMESPACE "ReferenceViewerFilters"

FLinearColor FReferenceViewerFilter::GetColor() const
{
	return FLinearColor(0.6f, 0.6f, 0.6f, 1);
}

FName FReferenceViewerFilter::GetIconName() const
{
	return NAME_None;
}

FReferenceViewerFilter_ShowCheckedOut::FReferenceViewerFilter_ShowCheckedOut(const TSharedPtr<FFilterCategory>& InCategory)
	: FReferenceViewerFilter(InCategory)
{
}

FString FReferenceViewerFilter_ShowCheckedOut::GetName() const
{
	return TEXT("RefViewer_CheckedOut");
}

FText FReferenceViewerFilter_ShowCheckedOut::GetDisplayName() const
{
	return LOCTEXT("ReferenceViewerFilter_ShowCheckedOutLabel", "Checked Out");
}

FText FReferenceViewerFilter_ShowCheckedOut::GetToolTipText() const
{
	return LOCTEXT("ReferenceViewerFilter_ShowCheckedOutTooltip", "Allow display of Checked Out Items.");
}

void FReferenceViewerFilter_ShowCheckedOut::ActiveStateChanged(bool bInActive)
{
	if (bInActive)
	{
		RequestStatus();
	}
}

bool FReferenceViewerFilter_ShowCheckedOut::PassesFilter(FReferenceNodeInfo& InItem) const
{
	if (!ISourceControlModule::Get().IsEnabled())
	{
		return false;
	}

	FSourceControlStatePtr SourceControlState =
		ISourceControlModule::Get().GetProvider().GetState(InItem.AssetData.GetPackage(), EStateCacheUsage::Use);

	return SourceControlState.IsValid() && (SourceControlState->IsCheckedOut() || SourceControlState->IsAdded());
}

void FReferenceViewerFilter_ShowCheckedOut::RequestStatus()
{
	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	if (ISourceControlModule::Get().IsEnabled())
	{
		// Request the opened files at filter construction time to make sure checked out files have the correct state for the filter
		TSharedRef<FUpdateStatus, ESPMode::ThreadSafe> UpdateStatusOperation =
			ISourceControlOperation::Create<FUpdateStatus>();
		UpdateStatusOperation->SetGetOpenedOnly(true);
		SourceControlProvider.Execute(
			UpdateStatusOperation,
			EConcurrency::Asynchronous,
			FSourceControlOperationComplete::CreateSP(this, &FReferenceViewerFilter_ShowCheckedOut::SourceControlOperationComplete)
		);
	}
}

void FReferenceViewerFilter_ShowCheckedOut::SourceControlOperationComplete(
	const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult
)
{
	BroadcastChangedEvent();
}


#undef LOCTEXT_NAMESPACE
