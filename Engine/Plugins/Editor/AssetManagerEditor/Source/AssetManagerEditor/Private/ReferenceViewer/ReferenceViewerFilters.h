// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ISourceControlOperation.h"
#include "ISourceControlProvider.h"
#include "Filters/FilterBase.h"

struct FReferenceNodeInfo;

class FReferenceViewerFilter : public FFilterBase<FReferenceNodeInfo&>
{
public:
	FReferenceViewerFilter(TSharedPtr<FFilterCategory> InCategory)
		: FFilterBase<FReferenceNodeInfo&>(MoveTemp(InCategory))
	{}

	//~ Begin FFilterBase
	/** Returns the color this filter button will be when displayed as a button */
	virtual FLinearColor GetColor() const override;

	/** Returns the name of the icon to use in menu entries */
	virtual FName GetIconName() const override;

	/** Returns true if the filter should be in the list when disabled and not in the list when enabled */
	virtual bool IsInverseFilter() const override
	{
		return false;
	}

	/** Notification that the filter became active or inactive */
	virtual void ActiveStateChanged(bool bActive) override {}

	/** Called when the right-click context menu is being built for this filter */
	virtual void ModifyContextMenu(FMenuBuilder& MenuBuilder) override {}

	/** Called when the state of a particular Content Browser is being saved to INI */
	virtual void SaveSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) const override {}

	/** Called when the state of a particular Content Browser is being loaded from INI */
	virtual void LoadSettings(const FString& IniFilename, const FString& IniSection, const FString& SettingsString) override {}
	//~ End FFilterBase
};

/**
 * A Filter to show nodes checked out by revision control
 */
class FReferenceViewerFilter_ShowCheckedOut final
	: public FReferenceViewerFilter
	, public TSharedFromThis<FReferenceViewerFilter_ShowCheckedOut>
{
public:
	FReferenceViewerFilter_ShowCheckedOut(const TSharedPtr<FFilterCategory>& InCategory);

	/** Returns the system name for this filter */
	virtual FString GetName() const override;

	/** Returns the human-readable name for this filter */
	virtual FText GetDisplayName() const override;

	/** Returns the tooltip for this filter, shown in the filters menu */
	virtual FText GetToolTipText() const override;

	/** Notification that the filter became active or inactive */
	virtual void ActiveStateChanged(bool bInActive) override;

	virtual bool PassesFilter(FReferenceNodeInfo& InItem) const override;

private:
	/** Request the source control status for this filter */
	void RequestStatus();

	/** Callback when source control operation has completed */
	void SourceControlOperationComplete(const FSourceControlOperationRef& InOperation, ECommandResult::Type InResult);
};
