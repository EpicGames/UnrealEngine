// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceViewer/ReferenceViewerSchema.h"
#include "SReferenceViewer.h"
#include "AssetManagerEditorCommands.h"
#include "AssetManagerEditorModule.h"
#include "ToolMenu.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "CollectionManagerModule.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "ICollectionSource.h"
#include "ConnectionDrawingPolicy.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "ReferenceViewer/EdGraph_ReferenceViewer.h"
#include "ReferenceViewer/EdGraphNode_Reference.h"
#include "ReferenceViewer/SReferenceViewer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReferenceViewerSchema)

namespace UE
{
namespace DependencyPinCategory
{
	FName NamePassive(TEXT("Passive"));
	FName NameHardUsedInGame(TEXT("Hard"));
	FName NameHardEditorOnly(TEXT("HardEditorOnly"));
	FName NameSoftUsedInGame(TEXT("Soft"));
	FName NameSoftEditorOnly(TEXT("SoftEditorOnly"));
	const FLinearColor ColorPassive = FLinearColor(128, 128, 128);
	const FLinearColor ColorHardUsedInGame = FLinearColor(FColor(236, 252, 227)); // RiceFlower
	const FLinearColor ColorHardEditorOnly = FLinearColor(FColor(118, 126, 114));
	const FLinearColor ColorSoftUsedInGame = FLinearColor(FColor(145, 66, 117)); // CannonPink
	const FLinearColor ColorSoftEditorOnly = FLinearColor(FColor(73, 33, 58));

}
}

EDependencyPinCategory ParseDependencyPinCategory(FName PinCategory)
{
	if (PinCategory == UE::DependencyPinCategory::NameHardUsedInGame)
	{
		return EDependencyPinCategory::LinkEndActive | EDependencyPinCategory::LinkTypeHard | EDependencyPinCategory::LinkTypeUsedInGame;
	}
	else if (PinCategory == UE::DependencyPinCategory::NameHardEditorOnly)
	{
		return EDependencyPinCategory::LinkEndActive | EDependencyPinCategory::LinkTypeHard;
	}
	else if (PinCategory == UE::DependencyPinCategory::NameSoftUsedInGame)
	{
		return EDependencyPinCategory::LinkEndActive | EDependencyPinCategory::LinkTypeUsedInGame;
	}
	else if (PinCategory == UE::DependencyPinCategory::NameSoftEditorOnly)
	{
		return EDependencyPinCategory::LinkEndActive;
	}
	else
	{
		return EDependencyPinCategory::LinkEndPassive;
	}
}

FName GetName(EDependencyPinCategory Category)
{
	if ((Category & EDependencyPinCategory::LinkEndMask) == EDependencyPinCategory::LinkEndPassive)
	{
		return UE::DependencyPinCategory::NamePassive;
	}
	else
	{
		switch (Category & EDependencyPinCategory::LinkTypeMask)
		{
		case EDependencyPinCategory::LinkTypeHard | EDependencyPinCategory::LinkTypeUsedInGame:
			return UE::DependencyPinCategory::NameHardUsedInGame;
		case EDependencyPinCategory::LinkTypeHard:
			return UE::DependencyPinCategory::NameHardEditorOnly;
		case EDependencyPinCategory::LinkTypeUsedInGame:
			return UE::DependencyPinCategory::NameSoftUsedInGame;
		default:
			return UE::DependencyPinCategory::NameSoftEditorOnly;
		}
	}
}


FLinearColor GetColor(EDependencyPinCategory Category)
{
	if ((Category & EDependencyPinCategory::LinkEndMask) == EDependencyPinCategory::LinkEndPassive)
	{
		return UE::DependencyPinCategory::ColorPassive;
	}
	else
	{
		switch (Category & EDependencyPinCategory::LinkTypeMask)
		{
		case EDependencyPinCategory::LinkTypeHard | EDependencyPinCategory::LinkTypeUsedInGame:
			return UE::DependencyPinCategory::ColorHardUsedInGame;
		case EDependencyPinCategory::LinkTypeHard:
			return UE::DependencyPinCategory::ColorHardEditorOnly;
		case EDependencyPinCategory::LinkTypeUsedInGame:
			return UE::DependencyPinCategory::ColorSoftUsedInGame;
		default:
			return UE::DependencyPinCategory::ColorSoftEditorOnly;
		}
	}
}

// Overridden connection drawing policy to use less curvy lines between nodes
class FReferenceViewerConnectionDrawingPolicy : public FConnectionDrawingPolicy
{
public:
	FReferenceViewerConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, FSlateWindowElementList& InDrawElements)
		: FConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements)
	{
	}

	virtual FVector2f ComputeSplineTangent(const FVector2f& Start, const FVector2f& End) const override
	{
		const int32 Tension = FMath::Abs<int32>(Start.X - End.X);
		return Tension * FVector2f(1.0f, 0);
	}

	virtual void DetermineWiringStyle(UEdGraphPin* OutputPin, UEdGraphPin* InputPin, /*inout*/ FConnectionParams& Params) override
	{
		EDependencyPinCategory OutputCategory = ParseDependencyPinCategory(OutputPin->PinType.PinCategory);
		EDependencyPinCategory InputCategory = ParseDependencyPinCategory(InputPin->PinType.PinCategory);

		EDependencyPinCategory Category = !!(OutputCategory & EDependencyPinCategory::LinkEndActive) ? OutputCategory : InputCategory;
		Params.WireColor = GetColor(Category);
	}
};

//////////////////////////////////////////////////////////////////////////
// UReferenceViewerSchema

UReferenceViewerSchema::UReferenceViewerSchema(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UReferenceViewerSchema::GetContextMenuActions(UToolMenu* Menu, UGraphNodeContextMenuContext* Context) const
{
	{
		FToolMenuSection& Section = Menu->AddSection(TEXT("Asset"), NSLOCTEXT("ReferenceViewerSchema", "AssetSectionLabel", "Asset"));
		Section.AddMenuEntry(FGlobalEditorCommonCommands::Get().FindInContentBrowser);
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().OpenSelectedInAssetEditor);
	}

	{
		FToolMenuSection& Section = Menu->AddSection(TEXT("Misc"), NSLOCTEXT("ReferenceViewerSchema", "MiscSectionLabel", "Misc"));
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().ZoomToFit);
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().ReCenterGraph);

		TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
		FCollectionManagerModule::GetModule().Get().GetVisibleCollectionContainers(CollectionContainers);

		if (!CollectionContainers.IsEmpty())
		{
			TWeakPtr<SReferenceViewer> ReferenceViewer;
			const UEdGraph_ReferenceViewer* Graph = Cast<const UEdGraph_ReferenceViewer>(ToRawPtr(Context->Graph));
			if (Graph != nullptr)
			{
				ReferenceViewer = Graph->GetReferenceViewer();
			}

			Section.AddSubMenu(
				"MakeCollectionWith",
				NSLOCTEXT("ReferenceViewerSchema", "MakeCollectionWithTitle", "Make Collection with"),
				NSLOCTEXT("ReferenceViewerSchema", "MakeCollectionWithTooltip", "Makes a collection with either the referencers or dependencies of the selected nodes."),
				FNewToolMenuDelegate::CreateUObject(this, &UReferenceViewerSchema::GetMakeCollectionWithSubMenu, MoveTemp(ReferenceViewer), MoveTemp(CollectionContainers))
			);
		}
	}

	{
		FToolMenuSection& Section = Menu->AddSection(TEXT("References"), NSLOCTEXT("ReferenceViewerSchema", "ReferencesSectionLabel", "References"));
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().CopyReferencedObjects);
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().CopyReferencingObjects);
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().ShowReferencedObjects);
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().ShowReferencingObjects);
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().ResolveReferencingProperties);
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().ShowReferenceTree);
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().ViewSizeMap);

		FToolMenuEntry ViewAssetAuditEntry = FToolMenuEntry::InitMenuEntry(FAssetManagerEditorCommands::Get().ViewAssetAudit);
		ViewAssetAuditEntry.Name = TEXT("ContextMenu");
		Section.AddEntry(ViewAssetAuditEntry);
	}

	// Our additional registered entries
	if (IAssetManagerEditorModule::IsAvailable())
	{
		const TArray<TSharedPtr<FUICommandInfo>> &CommandInfoList = IAssetManagerEditorModule::Get().GetRegisteredAdditionalReferenceViewerCommands();

		if (CommandInfoList.Num() > 0)
		{
			FToolMenuSection& Section = Menu->AddSection(TEXT("Extensions"), NSLOCTEXT("ReferenceViewerSchema", "ExtensionsSectionLabel", "Extensions"));

			for (TSharedPtr<FUICommandInfo> Entry : CommandInfoList)
			{
				if (Entry.IsValid())
				{
					Section.AddMenuEntry(Entry);
				}
			}
		}
	}
}

FLinearColor UReferenceViewerSchema::GetPinTypeColor(const FEdGraphPinType& PinType) const
{
	return GetColor(ParseDependencyPinCategory(PinType.PinCategory));
}

void UReferenceViewerSchema::BreakPinLinks(UEdGraphPin& TargetPin, bool bSendsNodeNotifcation) const
{
	// Don't allow breaking any links
}

void UReferenceViewerSchema::BreakSinglePinLink(UEdGraphPin* SourcePin, UEdGraphPin* TargetPin) const
{
	// Don't allow breaking any links
}

FPinConnectionResponse UReferenceViewerSchema::MovePinLinks(UEdGraphPin& MoveFromPin, UEdGraphPin& MoveToPin, bool bIsIntermediateMove, bool bNotifyLinkedNodes) const
{
	// Don't allow moving any links
	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, FString());
}

FPinConnectionResponse UReferenceViewerSchema::CopyPinLinks(UEdGraphPin& CopyFromPin, UEdGraphPin& CopyToPin, bool bIsIntermediateCopy) const
{
	// Don't allow copying any links
	return FPinConnectionResponse(CONNECT_RESPONSE_DISALLOW, FString());
}

FConnectionDrawingPolicy* UReferenceViewerSchema::CreateConnectionDrawingPolicy(int32 InBackLayerID, int32 InFrontLayerID, float InZoomFactor, const FSlateRect& InClippingRect, class FSlateWindowElementList& InDrawElements, class UEdGraph* InGraphObj) const
{
	return new FReferenceViewerConnectionDrawingPolicy(InBackLayerID, InFrontLayerID, InZoomFactor, InClippingRect, InDrawElements);
}

void UReferenceViewerSchema::DroppedAssetsOnGraph(const TArray<FAssetData>& Assets, const FVector2f& GraphPosition, UEdGraph* Graph) const
{
	TArray<FAssetIdentifier> AssetIdentifiers;

	IAssetManagerEditorModule::ExtractAssetIdentifiersFromAssetDataList(Assets, AssetIdentifiers);
	IAssetManagerEditorModule::Get().OpenReferenceViewerUI(AssetIdentifiers);
}

void UReferenceViewerSchema::GetAssetsGraphHoverMessage(const TArray<FAssetData>& Assets, const UEdGraph* HoverGraph, FString& OutTooltipText, bool& OutOkIcon) const
{
	OutOkIcon = true;
}

void UReferenceViewerSchema::GetMakeCollectionWithSubMenu(UToolMenu* Menu, TWeakPtr<SReferenceViewer> ReferenceViewer, TArray<TSharedPtr<ICollectionContainer>> CollectionContainers) const
{
	const TSharedRef<ICollectionContainer>& ProjectCollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

	FToolMenuSection& Section = Menu->AddSection("Section");

	auto CreateSubMenu = [this, &ReferenceViewer, &CollectionContainers, &ProjectCollectionContainer](bool bReferencers)
		{
			if (!ReferenceViewer.IsValid() || CollectionContainers.Num() == 1)
			{
				if (!ReferenceViewer.IsValid() || CollectionContainers[0] == ProjectCollectionContainer)
				{
					// Use the command version to show key bindings.
					return FNewToolMenuDelegate::CreateUObject(this, &UReferenceViewerSchema::GetMakeCollectionWithReferencersOrDependenciesSubMenu, bReferencers);
				}
				else
				{
					return FNewToolMenuDelegate::CreateUObject(this, &UReferenceViewerSchema::GetMakeCollectionWithReferencersOrDependenciesSubMenu, ReferenceViewer, CollectionContainers[0], bReferencers);
				}
			}
			else
			{
				// Create a sub menu to select the collection container.
				return FNewToolMenuDelegate::CreateUObject(this, &UReferenceViewerSchema::GetMakeCollectionWithCollectionContainersSubMenu, ReferenceViewer, CollectionContainers, bReferencers);
			}
		};

	Section.AddSubMenu(
		"MakeCollectionWithReferencers",
		NSLOCTEXT("ReferenceViewerSchema", "MakeCollectionWithReferencersTitle", "Referencers <-"),
		NSLOCTEXT("ReferenceViewerSchema", "MakeCollectionWithReferencersTooltip", "Makes a collection with assets one connection to the left of selected nodes."),
		CreateSubMenu(true)
		);

	Section.AddSubMenu(
		"MakeCollectionWithDependencies",
		NSLOCTEXT("ReferenceViewerSchema", "MakeCollectionWithDependenciesTitle", "Dependencies ->"),
		NSLOCTEXT("ReferenceViewerSchema", "MakeCollectionWithDependenciesTooltip", "Makes a collection with assets one connection to the right of selected nodes."),
		CreateSubMenu(false)
		);
}

void UReferenceViewerSchema::GetMakeCollectionWithCollectionContainersSubMenu(UToolMenu* Menu, TWeakPtr<SReferenceViewer> ReferenceViewer, TArray<TSharedPtr<ICollectionContainer>> CollectionContainers, bool bReferencers) const
{
	const TSharedRef<ICollectionContainer>& ProjectCollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

	FToolMenuSection& Section = Menu->AddSection(NAME_None, NSLOCTEXT("ReferenceViewerSchema", "MakeCollectionWithCollectionContainersMenuHeading", "Collection Containers"));

	for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
	{
		FNewToolMenuDelegate SubMenu;
		if (CollectionContainer == ProjectCollectionContainer)
		{
			// Use the command version to show key bindings.
			SubMenu = FNewToolMenuDelegate::CreateUObject(this, &UReferenceViewerSchema::GetMakeCollectionWithReferencersOrDependenciesSubMenu, bReferencers);
		}
		else
		{
			SubMenu = FNewToolMenuDelegate::CreateUObject(this, &UReferenceViewerSchema::GetMakeCollectionWithReferencersOrDependenciesSubMenu, ReferenceViewer, CollectionContainer, bReferencers);
		}
		Section.AddSubMenu(
			NAME_None,
			CollectionContainer->GetCollectionSource()->GetTitle(),
			TAttribute<FText>(),
			MoveTemp(SubMenu));
	}
}

void UReferenceViewerSchema::GetMakeCollectionWithReferencersOrDependenciesSubMenu(UToolMenu* Menu, bool bReferencers) const
{
	FToolMenuSection& Section = Menu->AddSection("Section");

	if (bReferencers)
	{
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().MakeLocalCollectionWithReferencers, 
			ECollectionShareType::ToText(ECollectionShareType::CST_Local),
			ECollectionShareType::GetDescription(ECollectionShareType::CST_Local), 
			FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Local))
			);
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().MakePrivateCollectionWithReferencers,
			ECollectionShareType::ToText(ECollectionShareType::CST_Private),
			ECollectionShareType::GetDescription(ECollectionShareType::CST_Private), 
			FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Private))
			);
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().MakeSharedCollectionWithReferencers,
			ECollectionShareType::ToText(ECollectionShareType::CST_Shared),
			ECollectionShareType::GetDescription(ECollectionShareType::CST_Shared), 
			FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Shared))
			);
	}
	else
	{
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().MakeLocalCollectionWithDependencies, 
			ECollectionShareType::ToText(ECollectionShareType::CST_Local),
			ECollectionShareType::GetDescription(ECollectionShareType::CST_Local), 
			FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Local))
			);
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().MakePrivateCollectionWithDependencies,
			ECollectionShareType::ToText(ECollectionShareType::CST_Private),
			ECollectionShareType::GetDescription(ECollectionShareType::CST_Private), 
			FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Private))
			);
		Section.AddMenuEntry(FAssetManagerEditorCommands::Get().MakeSharedCollectionWithDependencies,
			ECollectionShareType::ToText(ECollectionShareType::CST_Shared),
			ECollectionShareType::GetDescription(ECollectionShareType::CST_Shared), 
			FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Shared))
			);
	}
}

void UReferenceViewerSchema::GetMakeCollectionWithReferencersOrDependenciesSubMenu(UToolMenu* Menu, TWeakPtr<SReferenceViewer> ReferenceViewer, TSharedPtr<ICollectionContainer> CollectionContainer, bool bReferencers) const
{
	TSharedPtr<SReferenceViewer> PinnedReferenceViewer = ReferenceViewer.Pin();
	if (!PinnedReferenceViewer)
	{
		return;
	}

	FToolMenuSection& Section = Menu->AddSection("Section");

	Section.AddMenuEntry(NAME_None,
		ECollectionShareType::ToText(ECollectionShareType::CST_Local),
		ECollectionShareType::GetDescription(ECollectionShareType::CST_Local),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Local)),
		FToolUIActionChoice(FUIAction(
			FExecuteAction::CreateSP(PinnedReferenceViewer.ToSharedRef(), &SReferenceViewer::MakeCollectionWithReferencersOrDependencies, CollectionContainer, ECollectionShareType::CST_Local, bReferencers),
			FCanExecuteAction::CreateSP(PinnedReferenceViewer.ToSharedRef(), &SReferenceViewer::CanMakeCollectionWithReferencersOrDependencies, CollectionContainer, ECollectionShareType::CST_Local)))
	);
	Section.AddMenuEntry(NAME_None,
		ECollectionShareType::ToText(ECollectionShareType::CST_Private),
		ECollectionShareType::GetDescription(ECollectionShareType::CST_Private),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Private)),
		FToolUIActionChoice(FUIAction(
			FExecuteAction::CreateSP(PinnedReferenceViewer.ToSharedRef(), &SReferenceViewer::MakeCollectionWithReferencersOrDependencies, CollectionContainer, ECollectionShareType::CST_Private, bReferencers),
			FCanExecuteAction::CreateSP(PinnedReferenceViewer.ToSharedRef(), &SReferenceViewer::CanMakeCollectionWithReferencersOrDependencies, CollectionContainer, ECollectionShareType::CST_Private)))
	);
	Section.AddMenuEntry(NAME_None,
		ECollectionShareType::ToText(ECollectionShareType::CST_Shared),
		ECollectionShareType::GetDescription(ECollectionShareType::CST_Shared),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Shared)),
		FToolUIActionChoice(FUIAction(
			FExecuteAction::CreateSP(PinnedReferenceViewer.ToSharedRef(), &SReferenceViewer::MakeCollectionWithReferencersOrDependencies, CollectionContainer, ECollectionShareType::CST_Shared, bReferencers),
			FCanExecuteAction::CreateSP(PinnedReferenceViewer.ToSharedRef(), &SReferenceViewer::CanMakeCollectionWithReferencersOrDependencies, CollectionContainer, ECollectionShareType::CST_Shared)))
	);
}
