// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceViewer/ReferenceViewerSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ReferenceViewerSettings)

bool UReferenceViewerSettings::IsSearchDepthLimited() const
{
	return bLimitSearchDepth;
}

bool UReferenceViewerSettings::IsShowSoftReferences() const
{
	return bIsShowSoftReferences;
}

bool UReferenceViewerSettings::IsShowHardReferences() const
{
	return bIsShowHardReferences;
}

bool UReferenceViewerSettings::IsShowFilteredPackagesOnly() const
{
	return bIsShowFilteredPackagesOnly;
}

bool UReferenceViewerSettings::IsCompactMode() const
{
	return bIsCompactMode;
}

bool UReferenceViewerSettings::IsShowExternalReferencers() const
{
	return bIsShowExternalReferencers;
}

bool UReferenceViewerSettings::IsShowDuplicates() const
{
	return bIsShowDuplicates;
}

EEditorOnlyReferenceFilterType UReferenceViewerSettings::GetEditorOnlyReferenceFilterType() const
{
	return EditorOnlyReferenceFilterType;
}

bool UReferenceViewerSettings::IsShowManagementReferences() const
{
	return bIsShowManagementReferences;
}

bool UReferenceViewerSettings::IsShowSearchableNames() const
{
	return bIsShowSearchableNames;
}

bool UReferenceViewerSettings::IsShowCodePackages() const
{
	return bIsShowCodePackages;
}

bool UReferenceViewerSettings::IsShowReferencers() const
{
	return bIsShowReferencers;
}

bool UReferenceViewerSettings::IsShowDependencies() const
{
	return bIsShowDependencies;
}

void UReferenceViewerSettings::SetSearchDepthLimitEnabled(bool bNewEnabled)
{
	bLimitSearchDepth = bNewEnabled;
	SaveConfig();
}

void UReferenceViewerSettings::SetShowSoftReferencesEnabled(bool bNewEnabled)
{
	bIsShowSoftReferences = bNewEnabled;
	SaveConfig();
}

void UReferenceViewerSettings::SetShowHardReferencesEnabled(bool bNewEnabled)
{
	bIsShowHardReferences = bNewEnabled;
	SaveConfig();
}

void UReferenceViewerSettings::SetShowFilteredPackagesOnlyEnabled(bool bNewEnabled)
{
	bIsShowFilteredPackagesOnly = bNewEnabled;
	SaveConfig();
}

void UReferenceViewerSettings::SetCompactModeEnabled(bool bNewEnabled)
{
	bIsCompactMode = bNewEnabled;
	SaveConfig();
}

void UReferenceViewerSettings::SetShowExternalReferencersEnabled(bool bEnabled)
{
	bIsShowExternalReferencers = bEnabled;
	SaveConfig();
}

void UReferenceViewerSettings::SetShowDuplicatesEnabled(bool bNewEnabled)
{
	bIsShowDuplicates = bNewEnabled;
	SaveConfig();
}

void UReferenceViewerSettings::SetEditorOnlyReferenceFilterType(EEditorOnlyReferenceFilterType Value)
{
	EditorOnlyReferenceFilterType = Value;
	SaveConfig();
}

void UReferenceViewerSettings::SetShowManagementReferencesEnabled(bool bNewEnabled)
{
	bIsShowManagementReferences = bNewEnabled;
	SaveConfig();
}

void UReferenceViewerSettings::SetShowSearchableNames(bool bNewEnabled)
{
	bIsShowSearchableNames = bNewEnabled;
	SaveConfig();
}

void UReferenceViewerSettings::SetShowCodePackages(bool bNewEnabled)
{
	bIsShowCodePackages = bNewEnabled;
	SaveConfig();
}

void UReferenceViewerSettings::SetShowReferencers(const bool bNewEnabled)
{
	bIsShowReferencers = bNewEnabled;
	SaveConfig();
}

void UReferenceViewerSettings::SetShowDependencies(const bool bNewEnabled)
{
	bIsShowDependencies = bNewEnabled;
	SaveConfig();
}

int32 UReferenceViewerSettings::GetSearchDependencyDepthLimit() const
{
	return MaxSearchDependencyDepth;
}

void UReferenceViewerSettings::SetSearchDependencyDepthLimit(int32 NewDepthLimit, bool bSaveConfig)
{
	MaxSearchDependencyDepth = FMath::Max(NewDepthLimit, 0);
	if (bSaveConfig)
	{
		SaveConfig();
	}
}

int32 UReferenceViewerSettings::GetSearchReferencerDepthLimit() const
{
	return MaxSearchReferencerDepth;
}

void UReferenceViewerSettings::SetSearchReferencerDepthLimit(int32 NewDepthLimit, bool bSaveConfig)
{
	MaxSearchReferencerDepth = FMath::Max(NewDepthLimit, 0);
	if (bSaveConfig)
	{
		SaveConfig();
	}
}

int32 UReferenceViewerSettings::GetSearchBreadthLimit() const
{
	return MaxSearchBreadth;
}

void UReferenceViewerSettings::SetSearchBreadthLimit(int32 NewBreadthLimit)
{
	MaxSearchBreadth = FMath::Max(NewBreadthLimit, 0);
	SaveConfig();
}

bool UReferenceViewerSettings::GetEnableCollectionFilter() const
{
	return bEnableCollectionFilter;
}

void UReferenceViewerSettings::SetEnableCollectionFilter(bool bEnabled)
{
	bEnableCollectionFilter = bEnabled;
	SaveConfig();
}

bool UReferenceViewerSettings::GetEnablePluginFilter() const
{
	return bEnablePluginFilter;
}

void UReferenceViewerSettings::SetEnablePluginFilter(bool bEnabled)
{
	bEnablePluginFilter = bEnabled;
	SaveConfig();
}

bool UReferenceViewerSettings::IsShowPath() const
{
	return bIsShowPath;
}

void UReferenceViewerSettings::SetShowPathEnabled(bool bEnabled)
{
	bIsShowPath = bEnabled;
	SaveConfig();
}

bool UReferenceViewerSettings::GetFiltersEnabled() const
{
	return bFiltersEnabled;
}

void UReferenceViewerSettings::SetFiltersEnabled(bool bNewEnabled)
{
	bFiltersEnabled = bNewEnabled;
	SaveConfig();
}

bool UReferenceViewerSettings::GetFindPathEnabled() const
{
	return bFindPathEnabled;
}

void UReferenceViewerSettings::SetFindPathEnabled(bool bNewEnabled)
{
	bFindPathEnabled = bNewEnabled;
	SaveConfig();
}

bool UReferenceViewerSettings::AutoUpdateFilters() const
{
	return bAutoUpdateFilters;
}

void UReferenceViewerSettings::SetAutoUpdateFilters(bool bEnabled)
{
	bAutoUpdateFilters = bEnabled;
	SaveConfig();
}

const TArray<FilterState>& UReferenceViewerSettings::GetUserFilters() const
{
	return UserFilters;	
}

void UReferenceViewerSettings::SetUserFilters(TArray<FilterState>& InFilters)
{
	UserFilters = InFilters;
	SaveConfig();
}

const TMap<FString, bool>& UReferenceViewerSettings::GetCustomFilters() const
{
	return CustomFilters;
}

void UReferenceViewerSettings::SetCustomFilters(const TMap<FString, bool>& InCustomFilters)
{
	CustomFilters = InCustomFilters;
	SaveConfig();
}

