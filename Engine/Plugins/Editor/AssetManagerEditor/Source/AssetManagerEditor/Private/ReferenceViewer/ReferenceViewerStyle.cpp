// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceViewerStyle.h"

#include "Brushes/SlateImageBrush.h"
#include "Styling/SlateTypes.h"
#include "Brushes/SlateRoundedBoxBrush.h"
#include "Styling/SlateStyleMacros.h"
#include "Misc/Paths.h"
#include "Styling/StyleColors.h"
#include "Styling/CoreStyle.h"
#include "Styling/SlateStyleRegistry.h"


FName FReferenceViewerStyle::StyleName("ReferenceViewerStyle");

FReferenceViewerStyle::FReferenceViewerStyle()
	: FSlateStyleSet(StyleName)
{
	const FVector2D Icon16x16(16.0f, 16.0f);

	SetParentStyleName("EditorStyle");

	SetContentRoot(FPaths::EnginePluginsDir() / TEXT("Editor/AssetManagerEditor/Content"));
	SetCoreContentRoot(FPaths::EngineContentDir() / TEXT("Slate"));

	FScrollBarStyle ScrollBar = GetParentStyle()->GetWidgetStyle<FScrollBarStyle>("ScrollBar");
	FTextBlockStyle NormalText = GetParentStyle()->GetWidgetStyle<FTextBlockStyle>("NormalText");
	FEditableTextBoxStyle NormalEditableTextBoxStyle = GetParentStyle()->GetWidgetStyle<FEditableTextBoxStyle>("NormalEditableTextBox");

	FSlateFontInfo TitleFont = DEFAULT_FONT("Bold", 12);

	// Text Styles 
	FTextBlockStyle GraphNodeTitle = FTextBlockStyle(NormalText)
		.SetFont(TitleFont)
		.SetColorAndOpacity( FStyleColors::White)
		.SetShadowOffset( FVector2D::UnitVector )
		.SetShadowColorAndOpacity( FLinearColor::Black );
	Set( "Graph.Node.NodeTitle", GraphNodeTitle );


	Set( "Graph.Node.NodeTitleExtraLines", FTextBlockStyle(NormalText)
		.SetFont( DEFAULT_FONT( "Normal", 9 ) )
		.SetColorAndOpacity( FStyleColors::White)
		.SetShadowOffset( FVector2D::ZeroVector)
		.SetShadowColorAndOpacity( FLinearColor::Transparent)
	);


	FEditableTextBoxStyle GraphNodeTitleEditableText = FEditableTextBoxStyle(NormalEditableTextBoxStyle)
		.SetFont(TitleFont)
		.SetForegroundColor(FStyleColors::Input)
		.SetBackgroundImageNormal(FSlateRoundedBoxBrush(FStyleColors::Foreground, FStyleColors::Secondary, 1.0f))
		.SetBackgroundImageHovered(FSlateRoundedBoxBrush(FStyleColors::Foreground, FStyleColors::Hover, 1.0f))
		.SetBackgroundImageFocused(FSlateRoundedBoxBrush(FStyleColors::Foreground, FStyleColors::Primary, 1.0f))
		.SetBackgroundImageReadOnly(FSlateRoundedBoxBrush(FStyleColors::Header, FStyleColors::InputOutline, 1.0f))
		.SetForegroundColor(FStyleColors::White)
		.SetBackgroundColor(FStyleColors::White)
		.SetReadOnlyForegroundColor(FStyleColors::Foreground)
		.SetFocusedForegroundColor(FStyleColors::Background)
		.SetScrollBarStyle( ScrollBar );
	Set( "Graph.Node.NodeTitleEditableText", GraphNodeTitleEditableText );

	Set( "Graph.Node.NodeTitleInlineEditableText", FInlineEditableTextBlockStyle()
		.SetTextStyle(GraphNodeTitle)
		.SetEditableTextBoxStyle(GraphNodeTitleEditableText)
	);

	
	FLinearColor SpillColor(.3f, .3f, .3f, 1.0);
	int BodyRadius = 10.0; // Designed for 4 but using 10 to accomodate the shared selection border.  Update to 4 all nodes get aligned.

	// NOTE: 
	Set( "Graph.Node.BodyBackground", new FSlateRoundedBoxBrush(FStyleColors::Panel, BodyRadius));
	Set( "Graph.Node.BodyBorder",     new FSlateRoundedBoxBrush(SpillColor, BodyRadius));

	Set( "Graph.Node.Body",           new FSlateRoundedBoxBrush(FStyleColors::Panel, BodyRadius, FStyleColors::Transparent, 2.0));
	Set( "Graph.Node.ColorSpill",     new FSlateRoundedBoxBrush(SpillColor, FVector4(BodyRadius, BodyRadius, 0.0, 0.0)));

	Set( "Graph.Node.Duplicate",      new IMAGE_BRUSH_SVG("/GraphNode_Duplicate_8px", FVector2D(8.f, 8.f), FStyleColors::White));


	const FVector2D IconSize(20.0f, 20.0f);

	Set("Icons.ArrowLeft", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-left", IconSize));
	Set("Icons.ArrowRight", new CORE_IMAGE_BRUSH_SVG("Starship/Common/arrow-right", IconSize));

	Set("Icons.AutoFilters", new CORE_IMAGE_BRUSH_SVG("Starship/Common/FilterAuto", IconSize));
	Set("Icons.Filters", new CORE_IMAGE_BRUSH_SVG("Starship/Common/filter", IconSize));
	Set("Icons.Duplicate",      new IMAGE_BRUSH_SVG("/GraphNode_Duplicate_8px", IconSize, FStyleColors::White));

	// Centered Status Text
	{
		constexpr FLinearColor CenteredStatusColor = FLinearColor(1, 1, 1, 0.3f);
		constexpr float OutlineWidth = 2.0f;

		Set("Graph.CenteredStatusText", FTextBlockStyle(NormalText)
			.SetFont( DEFAULT_FONT( "BoldCondensed", 16 ) )
			.SetColorAndOpacity(CenteredStatusColor)
		);

		// A rounded box brush, showing only its border
		Set( "Graph.CenteredStatusBrush", new FSlateRoundedBoxBrush(FLinearColor::Transparent, BodyRadius, CenteredStatusColor, OutlineWidth));
	}

	// Referenced Properties Border
	{
		const FLinearColor OutlineColor = FColor::FromHex("#717171");
		const FLinearColor FillColor = FColor::FromHex("#282828");
		constexpr float OutlineWidth = 1.0f;

		// A rounded box brush, showing only its border
		Set( "Graph.ReferencedPropertiesBrush", new FSlateRoundedBoxBrush(FillColor, BodyRadius, OutlineColor, OutlineWidth));

		Set("Graph.ReferencedPropertiesText", FTextBlockStyle(NormalText)
			.SetColorAndOpacity(OutlineColor)
		);
	}

	// Referenced Properties List Rows
	{
		const FTableRowStyle AlternatingTableRowStyle = FTableRowStyle()
		   .SetEvenRowBackgroundBrush(FSlateNoResource())
		   .SetOddRowBackgroundBrush(FSlateNoResource());
		Set("Graph.ReferencedPropertiesTableRow", AlternatingTableRowStyle);
	}

	// Referenced Properties Close Button
	{
		const FButtonStyle CloseButton = FButtonStyle()
		   .SetNormal(CORE_IMAGE_BRUSH_SVG("Starship/Common/close-small", Icon16x16, FStyleColors::Foreground))
		   .SetPressed(CORE_IMAGE_BRUSH_SVG("Starship/Common/close-small", Icon16x16, FStyleColors::Foreground))
		   .SetHovered(CORE_IMAGE_BRUSH_SVG("Starship/Common/close-small", Icon16x16, FStyleColors::ForegroundHover));
		Set("Graph.ReferencedPropertiesCloseButton", CloseButton);
	}

	FSlateStyleRegistry::RegisterSlateStyle(*this);
}

FReferenceViewerStyle::~FReferenceViewerStyle()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*this);
}

FReferenceViewerStyle& FReferenceViewerStyle::Get()
{
	static FReferenceViewerStyle Inst;
	return Inst;
}


