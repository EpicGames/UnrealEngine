// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Styling/SlateStyle.h"

/**
 * The Reference Viewer is being used as a sandbox for testing out ui changes to graphs.
 * This style is setup to facilitate testing new styles for graphs and should be deleted
 * once the main graph style is updated, which currently resides in StarshipStyle (the Editor Style).
 * 
 * This style inherits from the EditorStyle (StarshipStyle) and therefore is just overwriting 
 * styles on the editor style.  No style names should have to be replaced when removing this style. 
 * However, someone will have to update the references from FReferenceViewerStyle::Get() to be FAppStyle::Get().
 */
class FReferenceViewerStyle
	: public FSlateStyleSet
{
public:
	static FName StyleName;

	/** Access the singleton instance for this style set */
	static FReferenceViewerStyle& Get();

private:

	FReferenceViewerStyle();
	~FReferenceViewerStyle();
};
