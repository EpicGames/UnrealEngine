// Copyright Epic Games, Inc. All Rights Reserved.

#include "SReferenceNode.h"
#include "ReferenceViewer/EdGraph_ReferenceViewer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "ReferenceViewer/EdGraphNode_Reference.h"
#include "AssetThumbnail.h"
#include "Widgets/Notifications/SErrorText.h"
#include "Widgets/Text/SInlineEditableTextBlock.h"
#include "SCommentBubble.h"
#include "ReferenceViewerStyle.h"

#define LOCTEXT_NAMESPACE "ReferenceViewer"

namespace FReferenceNode::Private
{
	static constexpr int32 ThumbnailSize = 128;
	static FVector2D CollapsedNodeTextBoxScale(ThumbnailSize * 0.7f, ThumbnailSize * 0.6f);
}

void SReferenceNode::Construct( const FArguments& InArgs, UEdGraphNode_Reference* InNode )
{
	using namespace FReferenceNode::Private;

	if (InNode->IsCollapsed())
	{
		bIsCollapsed = true;
	}
	else if (InNode->AllowsThumbnail())
	{
		if (InNode->UsesThumbnail())
		{
			// Create a thumbnail from the graph's thumbnail pool
			TSharedPtr<FAssetThumbnailPool> AssetThumbnailPool = InNode->GetReferenceViewerGraph()->GetAssetThumbnailPool();
			AssetThumbnail = MakeShareable( new FAssetThumbnail( InNode->GetAssetData(), ThumbnailSize, ThumbnailSize, AssetThumbnailPool ) );
		}
		else if (InNode->IsPackage())
		{
			// Just make a generic thumbnail
			AssetThumbnail = MakeShareable( new FAssetThumbnail( InNode->GetAssetData(), ThumbnailSize, ThumbnailSize, NULL ) );
		}
	}

	GraphNode = InNode;
	SetCursor( EMouseCursor::CardinalCross );
	UpdateGraphNode();
}

// UpdateGraphNode is similar to the base, but adds the option to hide the thumbnail */
void SReferenceNode::UpdateGraphNode()
{
	OutputPins.Empty();

	// Reset variables that are going to be exposed, in case we are refreshing an already setup node.
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	UpdateErrorInfo();

	//
	//             ______________________
	//            |      TITLE AREA      |
	//            +-------+------+-------+
	//            | (>) L |      | R (>) |
	//            | (>) E |      | I (>) |
	//            | (>) F |      | G (>) |
	//            | (>) T |      | H (>) |
	//            |       |      | T (>) |
	//            |_______|______|_______|
	//
	TSharedPtr<SVerticalBox> MainVerticalBox;
	TSharedPtr<SErrorText> ErrorText;
	TSharedPtr<SNodeTitle> NodeTitle = SNew(SNodeTitle, GraphNode).StyleSet(&FReferenceViewerStyle::Get());

	// Get node icon
	IconColor = FLinearColor::White;
	const FSlateBrush* IconBrush = nullptr;
	if (GraphNode != NULL && GraphNode->ShowPaletteIconOnNode())
	{
		IconBrush = GraphNode->GetIconAndTint(IconColor).GetOptionalIcon();
	}

	TSharedRef<SWidget> MainNodeWidget = SNullWidget::NullWidget;
	UEdGraphNode_Reference* RefGraphNode = CastChecked<UEdGraphNode_Reference>(GraphNode);
	bool bIsADuplicate = RefGraphNode->IsADuplicate();

	FLinearColor OpacityColor = RefGraphNode->GetIsFiltered() ? FLinearColor(1.0f, 1.0f, 1.0f, 0.4f) : FLinearColor::White;

	if (bIsCollapsed)
	{
		using namespace FReferenceNode::Private;

		MainNodeWidget =
			SNew(SBox)
			.MinDesiredWidth(CollapsedNodeTextBoxScale.X)
			.MinDesiredHeight(CollapsedNodeTextBoxScale.Y)
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.Font(FSlateFontInfo(FCoreStyle::GetDefaultFont(), 30))
				.ColorAndOpacity(FColor::White)
				.Text_Lambda([WeakGraphNode = TWeakObjectPtr<UEdGraphNode>(GraphNode)]()
				{
					if (UEdGraphNode* GraphNode = WeakGraphNode.Get())
					{
						return GraphNode->GetNodeTitle(ENodeTitleType::FullTitle);
					}

					return LOCTEXT("InvalidNodeTitle", "Invalid Node");
				})
			];
	}
	else if (AssetThumbnail.IsValid())
	{
		FAssetThumbnailConfig ThumbnailConfig;
		ThumbnailConfig.bAllowFadeIn = RefGraphNode->UsesThumbnail();
		ThumbnailConfig.bForceGenericThumbnail = !RefGraphNode->UsesThumbnail();
		ThumbnailConfig.AssetTypeColorOverride = FLinearColor::Transparent;
		ThumbnailConfig.AllowAssetStatusThumbnailOverlay = true;

		MainNodeWidget =
			SNew(SBox)
			.WidthOverride(AssetThumbnail->GetSize().X)
			.HeightOverride(AssetThumbnail->GetSize().Y)
			[
				AssetThumbnail->MakeThumbnailWidget(ThumbnailConfig)
			];
	}

	GetOrAddSlot( ENodeZone::Center )
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	[
		SAssignNew(MainVerticalBox, SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.ColorAndOpacity(OpacityColor)
			.BorderImage( FReferenceViewerStyle::Get().GetBrush( "Graph.Node.BodyBackground" ) )
			.Padding(0)
			[

			SNew(SBorder)
			.BorderImage( FReferenceViewerStyle::Get().GetBrush( "Graph.Node.BodyBorder" ) )
			.BorderBackgroundColor( this, &SReferenceNode::GetNodeTitleColor )
			.Padding(0)
			[

			SNew(SBorder)
			.BorderImage( FReferenceViewerStyle::Get().GetBrush( "Graph.Node.Body" ) )
			.Padding(0)
			[

				SNew(SVerticalBox)
				.ToolTipText( this, &SReferenceNode::GetNodeTooltip )

				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				.Padding(0)
				[
					SNew(SBorder)
					.Visibility(this, &SReferenceNode::GetCollapsedVisibility)
					.BorderImage( FReferenceViewerStyle::Get().GetBrush("Graph.Node.ColorSpill") )
					.Padding( FMargin(10.0f, 4.0f, 6.0f, 4.0f) )
					.BorderBackgroundColor( this, &SReferenceNode::GetNodeTitleColor )
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
						.AutoWidth()
						[
							SNew(SImage)
							.Image(IconBrush)
							.DesiredSizeOverride(FVector2D(24.0, 24.0))
							.ColorAndOpacity(this, &SGraphNode::GetNodeTitleIconColor)
						]
						+ SHorizontalBox::Slot()
						.FillWidth(1.0)
						.VAlign(VAlign_Center)
						[
							SNew(SVerticalBox)
							+SVerticalBox::Slot()
							.AutoHeight()
							.Padding(FMargin(0.f))
							.VAlign(VAlign_Center)
							[
								SAssignNew(InlineEditableText, SInlineEditableTextBlock)
								.Style( FReferenceViewerStyle::Get(), "Graph.Node.NodeTitleInlineEditableText" )
								.Text( NodeTitle.Get(), &SNodeTitle::GetHeadTitle )
								.OnVerifyTextChanged(this, &SReferenceNode::OnVerifyNameTextChanged)
								.OnTextCommitted(this, &SReferenceNode::OnNameTextCommited)
								.IsReadOnly( this, &SReferenceNode::IsNameReadOnly )
								.IsSelected(this, &SReferenceNode::IsSelectedExclusively)
							]
							+SVerticalBox::Slot()
							.AutoHeight()
							.Padding(FMargin(0.f))
							[
								NodeTitle.ToSharedRef()
							]
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Right)
						.VAlign(VAlign_Bottom)
						.Padding(FMargin(4.f, 0.f, 0.f, 1.f))
						[
							SNew(SImage)
							.Visibility(bIsADuplicate ? EVisibility::Visible : EVisibility::Hidden)
							.ToolTipText(LOCTEXT("DuplicateAsset", "This asset is referenced multiple times. Only the first occurance shows its decendants."))
							.DesiredSizeOverride(FVector2D(12.0, 12.0))
							.Image(FReferenceViewerStyle::Get().GetBrush("Graph.Node.Duplicate"))
							.ColorAndOpacity(FAppStyle::Get().GetColor("Colors.Foreground"))
						]
					]
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(1.0f)
				[
					// POPUP ERROR MESSAGE
					SAssignNew(ErrorText, SErrorText )
					.BackgroundColor( this, &SReferenceNode::GetErrorColor )
					.ToolTipText( this, &SReferenceNode::GetErrorMsgToolTip )
				]
				+SVerticalBox::Slot()
				.AutoHeight()
				.HAlign(HAlign_Fill)
				.VAlign(VAlign_Top)
				[
					// NODE CONTENT AREA
					SNew(SBorder)
					.BorderImage( FAppStyle::GetBrush("NoBorder") )
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					.Padding( FMargin(0,3) )
					[
						SNew(SHorizontalBox)
						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							// LEFT
							SNew(SBox)
							.WidthOverride(40)
							[
								SAssignNew(LeftNodeBox, SVerticalBox)
							]
						]

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Center)
						.FillWidth(1.0f)
						[
							// Thumbnail
							MainNodeWidget
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							// RIGHT
							SNew(SBox)
							.WidthOverride(40)
							[
								SAssignNew(RightNodeBox, SVerticalBox)
							]
						]
					]
				]

			] // Inner Background 
			] // Outline Border
			] // Background 
		]
	];
	// Create comment bubble if comment text is valid
	GetNodeObj()->bCommentBubbleVisible = !GetNodeObj()->NodeComment.IsEmpty();
	if( GetNodeObj()->ShouldMakeCommentBubbleVisible() && GetNodeObj()->bCommentBubbleVisible)
	{
		TSharedPtr<SCommentBubble> CommentBubble;

		SAssignNew(CommentBubble, SCommentBubble)
			.GraphNode(GraphNode)
			.Text(this, &SGraphNode::GetNodeComment);

		GetOrAddSlot( ENodeZone::TopCenter )
		.SlotOffset2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetOffset2f ))
		.SlotSize2f( TAttribute<FVector2f>( CommentBubble.Get(), &SCommentBubble::GetSize2f ))
		.AllowScaling( TAttribute<bool>( CommentBubble.Get(), &SCommentBubble::IsScalingAllowed ))
		.VAlign( VAlign_Top )
		[
			CommentBubble.ToSharedRef()
		];
	}

	ErrorReporting = ErrorText;
	ErrorReporting->SetError(ErrorMsg);
	CreateBelowWidgetControls(MainVerticalBox);

	CreatePinWidgets();
}

EVisibility SReferenceNode::GetCollapsedVisibility() const
{
	return bIsCollapsed ? EVisibility::Collapsed : EVisibility::Visible;
}

#undef LOCTEXT_NAMESPACE
