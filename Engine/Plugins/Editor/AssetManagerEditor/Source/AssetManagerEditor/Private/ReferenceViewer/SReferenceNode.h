// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SGraphNode.h"

class FAssetThumbnail;
class UEdGraphNode_Reference;

/**
 * 
 */
class SReferenceNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS( SReferenceNode ){}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs, UEdGraphNode_Reference* InNode );

	// SGraphNode implementation
	virtual void UpdateGraphNode() override;
	virtual bool IsNodeEditable() const override { return false; }
	// End SGraphNode implementation

private:

	/** Returns visibility setting for elements which should be hidden if node is collapsed */
	EVisibility GetCollapsedVisibility() const;

	TSharedPtr<class FAssetThumbnail> AssetThumbnail;

	/** Whether the current node is collapsed (representing more than one node) or not */
	bool bIsCollapsed = false;
};
