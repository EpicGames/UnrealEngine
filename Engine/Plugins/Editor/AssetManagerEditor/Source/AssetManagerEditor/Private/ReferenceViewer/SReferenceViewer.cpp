// Copyright Epic Games, Inc. All Rights Reserved.

#include "SReferenceViewer.h"
#include "Dialogs/Dialogs.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/PackageName.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Input/SSearchBox.h"
#include "ReferenceViewer/EdGraph_ReferenceViewer.h"
#include "Widgets/Input/SSpinBox.h"

#include "Algo/Sort.h"
#include "Algo/Unique.h"
#include "AssetToolsModule.h"
#include "Containers/VersePath.h"
#include "ReferenceViewer/HistoryManager.h"
#include "ReferenceViewerStyle.h"
#include "ReferenceViewer/EdGraphNode_Reference.h"
#include "ReferenceViewer/EdGraphNode_ReferencedProperties.h"
#include "ReferenceViewer/ReferenceViewerSchema.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IAssetTools.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "ICollectionSource.h"
#include "CollectionManagerModule.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "Editor.h"
#include "AssetManagerEditorCommands.h"
#include "EditorWidgetsModule.h"
#include "ReferenceViewer/ReferenceViewerSettings.h"
#include "Settings/EditorProjectSettings.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "Engine/AssetManager.h"
#include "ReferenceViewer/SReferenceViewerFilterBar.h"
#include "ReferenceViewer/SReferencedPropertiesNode.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SComboButton.h"
#include "HAL/PlatformApplicationMisc.h"
#include "AssetManagerEditorModule.h"

#include "ObjectTools.h"
#include "Selection.h"
#include "Interfaces/IPluginManager.h"
#include "StatusBarSubsystem.h"

#define LOCTEXT_NAMESPACE "ReferenceViewer"

bool DoesAssetPassSearchTextFilter(const FAssetIdentifier& InNode, const FAssetData& AssetData, bool bShowingContentVersePath, const TArray<FString>& InSearchWords)
{
	FString NodeString = InNode.ToString();
	UE::Core::FVersePath VersePath;
	if (bShowingContentVersePath)
	{
		VersePath = AssetData.GetVersePath();
	}
	for (const FString& Word : InSearchWords)
	{
		if (!NodeString.Contains(Word) && !VersePath.ToString().Contains(Word))
		{
			return false;
		}
	}

	return true;
}

EAppReturnType::Type ShowAssetsNeedsToLoadMessage(const TSet<FAssetData>& UnloadedAssetsData, bool bShowingContentVersePath)
{
	FString UnloadedAssetsNames;

	int32 Count = 0;
	constexpr int32 MaxAssetsShown = 5;
	for (const FAssetData& Data : UnloadedAssetsData)
	{
		// Don't show more than 5 entries
		if (Count++ > MaxAssetsShown - 1)
		{
			break;
		}

		UnloadedAssetsNames += TEXT("\n\n");

		if (bShowingContentVersePath)
		{
			const UE::Core::FVersePath VersePath = Data.GetVersePath();
			if (VersePath.IsValid())
			{
				UnloadedAssetsNames += Data.AssetClassPath.ToString();
				UnloadedAssetsNames += TEXT(' ');
				UnloadedAssetsNames += VersePath.ToString();
				continue;
			}
		}

		UnloadedAssetsNames += Data.GetFullName();
	}

	if (UnloadedAssetsData.Num() > MaxAssetsShown)
	{
		const int32 HiddenAssets = UnloadedAssetsData.Num() - 5;

		FString HiddenAssetsString = TEXT("and ") + FString::FromInt(HiddenAssets) + TEXT(" more...");
		UnloadedAssetsNames += TEXT("\n\n") + HiddenAssetsString;
	}

	FFormatNamedArguments Args;
	Args.Add(TEXT("UnloadedAssets"), FText::FromString(UnloadedAssetsNames));
	static FText MessageTitle(
		LOCTEXT("ReferencingProperties_AssetsNeedLoadingTitle", "Resolve Referencing Properties: Assets Loading")
	);

	return FMessageDialog::Open(
		EAppMsgType::OkCancel,
		FText::Format(
			LOCTEXT(
				"ReferencingProperties_AssetsNeedLoading", "The following Assets will be loaded in order to resolve referencing properties for the selected nodes: \n {UnloadedAssets}\n\n Do you wish to continue?"
			),
			Args
		),
		MessageTitle
	);
}

SReferenceViewer::~SReferenceViewer()
{
	Settings->SetFindPathEnabled(false); 

	if (!GExitPurge)
	{
		if ( ensure(GraphObj) )
		{
			GraphObj->RemoveFromRoot();
		}
	}
}

void SReferenceViewer::Construct(const FArguments& InArgs)
{
	bShowingContentVersePath = FAssetToolsModule::GetModule().Get().ShowingContentVersePath();
	bRebuildingFilters = false;
	bNeedsGraphRebuild = false;
	bNeedsGraphRefilter = false;
	bNeedsReferencedPropertiesUpdate = false;
	Settings = GetMutableDefault<UReferenceViewerSettings>();

	// Create an action list and register commands
	RegisterActions();

	// Set up the history manager
	HistoryManager.SetOnApplyHistoryData(FOnApplyHistoryData::CreateSP(this, &SReferenceViewer::OnApplyHistoryData));
	HistoryManager.SetOnUpdateHistoryData(FOnUpdateHistoryData::CreateSP(this, &SReferenceViewer::OnUpdateHistoryData));
	
	// Create the graph
	GraphObj = NewObject<UEdGraph_ReferenceViewer>();
	GraphObj->Schema = UReferenceViewerSchema::StaticClass();
	GraphObj->AddToRoot();
	GraphObj->SetReferenceViewer(StaticCastSharedRef<SReferenceViewer>(AsShared()));
	GraphObj->SetShowingContentVersePath(bShowingContentVersePath);
	GraphObj->OnAssetsChanged().AddSP(this, &SReferenceViewer::OnUpdateFilterBar);

	SGraphEditor::FGraphEditorEvents GraphEvents;
	GraphEvents.OnNodeDoubleClicked = FSingleNodeEvent::CreateSP(this, &SReferenceViewer::OnNodeDoubleClicked);
	GraphEvents.OnCreateActionMenuAtLocation = SGraphEditor::FOnCreateActionMenuAtLocation::CreateSP(this, &SReferenceViewer::OnCreateGraphActionMenu);

	// Create the graph editor
	GraphEditorPtr = SNew(SGraphEditor)
		.AdditionalCommands(ReferenceViewerActions)
		.GraphToEdit(GraphObj)
		.GraphEvents(GraphEvents)
		.ShowGraphStateOverlay(false)
		.OnNavigateHistoryBack(FSimpleDelegate::CreateSP(this, &SReferenceViewer::GraphNavigateHistoryBack))
		.OnNavigateHistoryForward(FSimpleDelegate::CreateSP(this, &SReferenceViewer::GraphNavigateHistoryForward));

	FEditorWidgetsModule& EditorWidgetsModule = FModuleManager::LoadModuleChecked<FEditorWidgetsModule>("EditorWidgets");
	TSharedRef<SWidget> AssetDiscoveryIndicator = EditorWidgetsModule.CreateAssetDiscoveryIndicator(EAssetDiscoveryIndicatorScaleMode::Scale_None, FMargin(16, 8), false);

	const FAssetManagerEditorCommands& UICommands	= FAssetManagerEditorCommands::Get();

	static const FName DefaultForegroundName("DefaultForeground");

	// Visual options visibility
	FixAndHideSearchDepthLimit = 0;
	FixAndHideSearchBreadthLimit = 0;
	bShowCollectionFilter = true;
	bShowPluginFilter = true;
	bShowShowReferencesOptions = true;
	bShowShowSearchableNames = true;
	bShowShowCodePackages = true;
	bShowShowFilteredPackagesOnly = true;
	bShowCompactMode = true;
	bDirtyResults = false;

	// Retrieve and apply Breadth limit and show searchable names values from Project Settings
	if (const UEditorProjectAppearanceSettings* DefaultProjectAppearanceSettings = GetDefault<UEditorProjectAppearanceSettings>())
	{
		FixAndHideSearchBreadthLimit = DefaultProjectAppearanceSettings->ReferenceViewerDefaultMaxSearchBreadth;

		switch (DefaultProjectAppearanceSettings->ShowSearchableNames)
		{
			case EReferenceViewerSettingMode::NoPreference:
				bShowShowSearchableNames = true;
				break;

			case EReferenceViewerSettingMode::ShowByDefault:
				bShowShowSearchableNames = true;
				break;

			case EReferenceViewerSettingMode::HideByDefault:
				bShowShowSearchableNames = false;
				break;

			default:
				bShowShowSearchableNames = true;
				break;
		}

		if (Settings)
		{
			Settings->SetSearchBreadthLimit(FixAndHideSearchBreadthLimit);
			Settings->SetShowSearchableNames(bShowShowSearchableNames);
		}
	}

	// Build the status bar widget if a parent tab was provided
	TSharedRef<SWidget> StatusBarWidget = SNullWidget::NullWidget;
	if (InArgs._ParentTab.IsValid())
	{
		const FName StatusBarName(FString::Printf(TEXT("ReferenceViewer_0x%llx"), reinterpret_cast<uint64>(this)));
		if (UStatusBarSubsystem* StatusBarSubsystem = GEditor->GetEditorSubsystem<UStatusBarSubsystem>())
		{
			StatusBarWidget = StatusBarSubsystem->MakeStatusBarWidget(StatusBarName, InArgs._ParentTab.ToSharedRef());
			StatusBarWidget->SetVisibility(TAttribute<EVisibility>::CreateLambda([ParentTab = TWeakPtr<SDockTab>(InArgs._ParentTab)]() { return (ParentTab.IsValid() && ParentTab.Pin()->GetVisualTabRole() == ETabRole::MajorTab) ? EVisibility::Visible : EVisibility::Collapsed; }));
		}
	}

	SAssignNew(FilterWidget, SReferenceViewerFilterBar)
		.Visibility_Lambda([this]() { return !Settings->GetFiltersEnabled() ? EVisibility::Collapsed : EVisibility::Visible; })
		.OnConvertItemToAssetData_Lambda([this] (FReferenceNodeInfo& InNodeInfo, FAssetData& OutAssetData) -> bool { 
			OutAssetData = InNodeInfo.AssetData; 
			return true; 
		})
		.UseDefaultAssetFilters(true)
		.OnFilterChanged_Lambda([this] { 
			if (!bRebuildingFilters && GraphObj)
			{
				GraphObj->SetCurrentFilterCollection(FilterWidget->GetAllActiveFilters());
				GraphObj->RefilterGraph();
				FilterWidget->SaveSettings();
			}
		})
	;

	TSharedPtr<SWidget> FilterCombo = FilterWidget->MakeAddFilterButton(FilterWidget.ToSharedRef());
	FilterCombo->SetVisibility(TAttribute<EVisibility>::CreateLambda([this]() { return !Settings->GetFiltersEnabled() ? EVisibility::Collapsed : EVisibility::Visible; }));

	ChildSlot
	[

		SNew(SVerticalBox)

		// Path and history
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("Brushes.Panel"))
			[
				SNew(SHorizontalBox)

				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(4, 0)
				[
					MakeToolBar()
				]

				// Path
				+SHorizontalBox::Slot()
				.Padding(0, 0, 4, 0)
				.FillWidth(1.f)
				.VAlign(VAlign_Center)
				[
					SNew(SBorder)
					.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
					[
						SNew(SEditableTextBox)
						.Text(this, &SReferenceViewer::GetAddressBarText)
						.OnTextCommitted(this, &SReferenceViewer::OnAddressBarTextCommitted)
						.OnTextChanged(this, &SReferenceViewer::OnAddressBarTextChanged)
						.SelectAllTextWhenFocused(true)
						.SelectAllTextOnCommit(true)
						.Style(FAppStyle::Get(), "ReferenceViewer.PathText")
					]
				]
				+SHorizontalBox::Slot()
				.Padding(0, 7, 4, 8)
				.FillWidth(1.0)
				.VAlign(VAlign_Fill)
				[
					SAssignNew(FindPathAssetPicker, SComboButton)
					.OnGetMenuContent(this, &SReferenceViewer::GenerateFindPathAssetPickerMenu)
					.Visibility_Lambda([this]() { return !Settings->GetFindPathEnabled() ? EVisibility::Collapsed : EVisibility::Visible; })
					.ButtonContent()
					[
						SNew(STextBlock)
						.Text_Lambda([this] { return FindPathAssetId.IsValid() ? GetIdentifierText(FindPathAssetId) : LOCTEXT("ChooseTargetAsset", "Choose a target asset ... "); } )
					]
				]
			]
		]

		// Graph
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		.HAlign(HAlign_Fill)
		[
			SNew(SOverlay)

			+SOverlay::Slot()
			[
				GraphEditorPtr.ToSharedRef()
			]

			+SOverlay::Slot()
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Fill)
			[
				SNew(SImage)
				.Image(FAppStyle::GetBrush("Brushes.Recessed"))
				.ColorAndOpacity_Lambda( [this] () { return bNeedsGraphRebuild ? FLinearColor(1.0, 1.0, 1.0, 0.25) : FLinearColor::Transparent; } )
				.Visibility(EVisibility::HitTestInvisible)
			]

			+SOverlay::Slot()
			.VAlign(VAlign_Top)
			.HAlign(HAlign_Fill)
			.Padding(8)
			[
				SNew(SHorizontalBox)
				.Visibility_Lambda([this]() { return ( Settings->GetFindPathEnabled() ? EVisibility::Collapsed : EVisibility::SelfHitTestInvisible); })

				+SHorizontalBox::Slot()
				.AutoWidth()
				[

				SNew(SBorder)
				.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
				[
					SNew(SVerticalBox)
					+SVerticalBox::Slot()
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Center)
					.Padding(2.f)
					.AutoHeight()
					[
						SAssignNew(SearchBox, SSearchBox)
						.HintText(LOCTEXT("Search", "Search..."))
						.ToolTipText(LOCTEXT("SearchTooltip", "Type here to search (pressing Enter zooms to the results)"))
						.OnTextChanged(this, &SReferenceViewer::HandleOnSearchTextChanged)
						.OnTextCommitted(this, &SReferenceViewer::HandleOnSearchTextCommitted)
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.Visibility_Lambda([this]() { return (FixAndHideSearchDepthLimit > 0 ? EVisibility::Collapsed : EVisibility::Visible); })

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(2.f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("SearchDepthReferencersLabelText", "Search Referencers Depth"))
							.ToolTipText(FText::Format( LOCTEXT("ReferenceDepthToolTip", "Adjust Referencer Search Depth (+/-):  {0} / {1}\nSet Referencer Search Depth:                        {2}"),
															UICommands.IncreaseReferencerSearchDepth->GetInputText().ToUpper(),
															UICommands.DecreaseReferencerSearchDepth->GetInputText().ToUpper(),
															UICommands.SetReferencerSearchDepth->GetInputText().ToUpper()))

						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(2.f)
						[
							SNew(SBox)
							.WidthOverride(100)
							[
								SAssignNew(ReferencerCountBox, SSpinBox<int32>)
								.Value(this, &SReferenceViewer::GetSearchReferencerDepthCount)
								.OnValueChanged_Lambda([this] (int32 NewValue)
									{
										if (NewValue != Settings->GetSearchReferencerDepthLimit())
										{
											Settings->SetSearchReferencerDepthLimit(NewValue, false);
											bNeedsGraphRebuild = true;

											SliderDelayLastMovedTime = FSlateApplication::Get().GetCurrentTime();
										}
									}
								)
								.OnValueCommitted_Lambda([this] (int32 NewValue, ETextCommit::Type CommitType) 
									{ 
										FSlateApplication::Get().SetKeyboardFocus(GraphEditorPtr, EFocusCause::SetDirectly); 

										if (NewValue != Settings->GetSearchReferencerDepthLimit() || bNeedsGraphRebuild)
										{
											Settings->SetSearchReferencerDepthLimit(NewValue, false); 
											bNeedsGraphRebuild = false;
											RebuildGraph();
										}

										// Always save the config since we explicitly did not save during slider movement to preserve interactivity
										Settings->SaveConfig();
									} 
								)
								.MinValue(0)
								.MaxValue(50)
								.MaxSliderValue(10)
							]
						]
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.Visibility_Lambda([this]() { return (FixAndHideSearchDepthLimit > 0 ? EVisibility::Collapsed : EVisibility::Visible); })

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.Padding(2.f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("SearchDepthDependenciesLabelText", "Search Dependencies Depth"))
							.ToolTipText(FText::Format( LOCTEXT("DependencyDepthToolTip", "Adjust Dependency Search Depth (+/-):  {0} / {1}\nSet Dependency Search Depth:                        {2}"),
															UICommands.IncreaseDependencySearchDepth->GetInputText().ToUpper(),
															UICommands.DecreaseDependencySearchDepth->GetInputText().ToUpper(),
															UICommands.SetDependencySearchDepth->GetInputText().ToUpper()))
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(2.f)
						[
							SNew(SBox)
							.WidthOverride(100)
							[
								SAssignNew(DependencyCountBox, SSpinBox<int32>)
								.Value(this, &SReferenceViewer::GetSearchDependencyDepthCount)
								.OnValueChanged_Lambda([this] (int32 NewValue)
									{	
										if (NewValue != Settings->GetSearchDependencyDepthLimit())
										{
											Settings->SetSearchDependencyDepthLimit(NewValue, false);
											bNeedsGraphRebuild = true;

											SliderDelayLastMovedTime = FSlateApplication::Get().GetCurrentTime();
										}
									}
								)
								.OnValueCommitted_Lambda([this] (int32 NewValue, ETextCommit::Type CommitType) 
									{ 
										FSlateApplication::Get().SetKeyboardFocus(GraphEditorPtr, EFocusCause::SetDirectly); 

										if (NewValue != Settings->GetSearchDependencyDepthLimit() || bNeedsGraphRebuild)
										{
											Settings->SetSearchDependencyDepthLimit(NewValue, false);
											bNeedsGraphRebuild = false;
											RebuildGraph();
										}

										// Always save the config since we explicitly did not save during slider movement to preserve interactivity
										Settings->SaveConfig();
									}
								)
								.MinValue(0)
								.MaxValue(50)
								.MaxSliderValue(10)
							]
						]
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.Visibility_Lambda([this]() { return (FixAndHideSearchBreadthLimit > 0 ? EVisibility::Collapsed : EVisibility::Visible); })

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.AutoWidth()
						.Padding(2.f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("SearchBreadthLabelText", "Search Breadth Limit"))
							.ToolTipText(FText::Format( LOCTEXT("BreadthLimitToolTip", "Adjust Breadth Limit (+/-):  {0} / {1}\nSet Breadth Limit:                        {2}"),
															UICommands.IncreaseBreadth->GetInputText().ToUpper(),
															UICommands.DecreaseBreadth->GetInputText().ToUpper(),
															UICommands.SetBreadth->GetInputText().ToUpper()))
						]

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						.FillWidth(1.0)
						.Padding(2.f, 0.f, 8.f, 0.f)
						[
							SNew(SImage)
							.ToolTipText(LOCTEXT("BreadthLimitReachedToolTip", "The Breadth Limit was reached."))
							.Image(FAppStyle::GetBrush("Icons.WarningWithColor"))
							.Visibility_Lambda([this] { return GraphObj && GraphObj->BreadthLimitExceeded() ? EVisibility::Visible : EVisibility::Hidden; })
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(2.f)
						[
							SNew(SBox)
							.WidthOverride(100)
							[
								SAssignNew(BreadthLimitBox, SSpinBox<int32>)
								.Value(this, &SReferenceViewer::GetSearchBreadthCount)
								.OnValueChanged(this, &SReferenceViewer::OnSearchBreadthChanged)
								.OnValueCommitted(this, &SReferenceViewer::OnSearchBreadthCommited)
								.MinValue(1)
								.MaxValue(1000)
								.MaxSliderValue(1000)
							]
						]
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.Visibility_Lambda([this]() { return (bShowCollectionFilter ? EVisibility::Visible : EVisibility::Collapsed); })

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.FillWidth(1.0)
						.Padding(2.f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("CollectionFilter", "Collection Filter"))
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(2.f)
						[
							SNew(SCheckBox)
							.OnCheckStateChanged( this, &SReferenceViewer::OnEnableCollectionFilterChanged )
							.IsChecked( this, &SReferenceViewer::IsEnableCollectionFilterChecked )
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(2.f)
						[
							SNew(SBox)
							.WidthOverride(100)
							[
								SNew(SComboButton)
								.OnGetMenuContent(this, &SReferenceViewer::BuildCollectionFilterMenu)
								.ButtonContent()
								[
									SNew(STextBlock)
									.Text(this, &SReferenceViewer::GetCollectionComboButtonText)
									.ToolTipText(this, &SReferenceViewer::GetCollectionComboButtonText)
								]
							]
						]
					]

					+SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SHorizontalBox)
						.Visibility_Lambda([this]() { return (bShowPluginFilter ? EVisibility::Visible : EVisibility::Collapsed); })

						+SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.FillWidth(1.0)
						.Padding(2.f)
						[
							SNew(STextBlock)
							.Text(LOCTEXT("PluginFilter", "Plugin Filter"))
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(2.f)
						[
							SNew(SCheckBox)
							.OnCheckStateChanged( this, &SReferenceViewer::OnEnablePluginFilterChanged )
							.IsChecked( this, &SReferenceViewer::IsEnablePluginFilterChecked )
						]

						+SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.Padding(2.f)
						[
							SNew(SBox)
							.WidthOverride(100)
							[
								SNew(SComboButton)
								.OnGetMenuContent(this, &SReferenceViewer::BuildPluginFilterMenu)
								.ButtonContent()
								[
									SNew(STextBlock)
									.Text(this, &SReferenceViewer::GetPluginComboButtonText)
									.ToolTipText(this, &SReferenceViewer::GetPluginComboButtonText)
								]
							]
						]
					]
				]
				] // SHorizontalBox::Slot()

				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Top)
				[
					FilterCombo.ToSharedRef()
				]

				+SHorizontalBox::Slot()
				.FillWidth(1.0)
				.VAlign(VAlign_Top)
				[
					FilterWidget.ToSharedRef()
				]
			]

			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(FMargin(24, 0, 24, 0))
			[
				AssetDiscoveryIndicator
			]

			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Bottom)
			.Padding(FMargin(0, 0, 0, 16))
			[
				SNew(STextBlock)
				.Text(this, &SReferenceViewer::GetStatusText)
			]

			+SOverlay::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(FMargin(0, 0, 0, 16))
			[
				SNew(SBox)
				.MinDesiredWidth(325.0f)
				.MinDesiredHeight(50.0f)
				[
					// Show text within a rounded border
					SNew(SBorder)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.BorderImage(FReferenceViewerStyle::Get().GetBrush("Graph.CenteredStatusBrush"))
					.Visibility(this, &SReferenceViewer::GetCenteredStatusVisibility)
					[
						SNew(STextBlock)
						.Justification(ETextJustify::Center)
						.TextStyle(FReferenceViewerStyle::Get(), "Graph.CenteredStatusText")
						.Text(this, &SReferenceViewer::GetCenteredStatusText)
					]
				]
			]
		]

		// Status bar
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			StatusBarWidget
		]
	];

	SetCanTick(true);
}

void SReferenceViewer::Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime )
{
	{
		bool bNewShowingContentVersePath = FAssetToolsModule::GetModule().Get().ShowingContentVersePath();
		if (bShowingContentVersePath != bNewShowingContentVersePath)
		{
			bShowingContentVersePath = bNewShowingContentVersePath;

			if (GraphObj)
			{
				GraphObj->SetShowingContentVersePath(bShowingContentVersePath);
			}

			UpdateIsPassingSearchFilterCallback();
		}
	}

	if (bNeedsGraphRebuild && (InCurrentTime - SliderDelayLastMovedTime > GraphRebuildSliderDelay))
	{
		bNeedsGraphRebuild = false;
		RebuildGraph();
	}

	if (bNeedsGraphRefilter)
	{
		bNeedsGraphRefilter = false;
		if (GraphObj)
		{
			GraphObj->RefilterGraph();
		}
	}

	if (bNeedsReferencedPropertiesUpdate)
	{
		bNeedsReferencedPropertiesUpdate = false;
		GraphObj->RefreshReferencedPropertiesNodes();
	}
}

FReply SReferenceViewer::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	return ReferenceViewerActions->ProcessCommandBindings(InKeyEvent) ? FReply::Handled() : FReply::Unhandled();
}

void SReferenceViewer::SetGraphRootIdentifiers(const TArray<FAssetIdentifier>& NewGraphRootIdentifiers, const FReferenceViewerParams& ReferenceViewerParams)
{
	GraphObj->SetGraphRoot(NewGraphRootIdentifiers);
	// Set properties
	Settings->SetShowReferencers(ReferenceViewerParams.bShowReferencers);
	Settings->SetShowDependencies(ReferenceViewerParams.bShowDependencies);
	// Set user-interactive properties
	FixAndHideSearchDepthLimit = ReferenceViewerParams.FixAndHideSearchDepthLimit;
	if (FixAndHideSearchDepthLimit > 0)
	{
		Settings->SetSearchDependencyDepthLimit(FixAndHideSearchDepthLimit);
		Settings->SetSearchReferencerDepthLimit(FixAndHideSearchDepthLimit);
		Settings->SetSearchDepthLimitEnabled(true);
	}
	FixAndHideSearchBreadthLimit = ReferenceViewerParams.FixAndHideSearchBreadthLimit;
	if (FixAndHideSearchBreadthLimit > 0)
	{
		Settings->SetSearchBreadthLimit(FixAndHideSearchBreadthLimit);
	}
	bShowCollectionFilter = ReferenceViewerParams.bShowCollectionFilter;
	bShowPluginFilter = ReferenceViewerParams.bShowPluginFilter;
	bShowShowReferencesOptions = ReferenceViewerParams.bShowShowReferencesOptions;
	bShowShowSearchableNames = ReferenceViewerParams.bShowShowSearchableNames;
	bShowShowCodePackages = ReferenceViewerParams.bShowShowCodePackages;

	bShowShowFilteredPackagesOnly = ReferenceViewerParams.bShowShowFilteredPackagesOnly;
	if (ReferenceViewerParams.bShowFilteredPackagesOnly.IsSet())
	{
		Settings->SetShowFilteredPackagesOnlyEnabled(ReferenceViewerParams.bShowFilteredPackagesOnly.GetValue());
	}
	

	bShowCompactMode = ReferenceViewerParams.bShowCompactMode;
	if (ReferenceViewerParams.bCompactMode.IsSet())
	{
		Settings->SetCompactModeEnabled(ReferenceViewerParams.bCompactMode.GetValue());
	}

	if (Settings->IsShowManagementReferences())
	{
		UAssetManager::Get().UpdateManagementDatabase();
	}

	if (!ReferenceViewerParams.PluginFilter.IsEmpty())
	{
		Settings->SetEnablePluginFilter(true);
		GraphObj->SetCurrentPluginFilter(ReferenceViewerParams.PluginFilter);
	}

	RebuildGraph();

	UpdateIsPassingSearchFilterCallback();

	// Zoom once this frame to make sure widgets are visible, then zoom again so size is correct
	TriggerZoomToFit(0, 0);
	RegisterActiveTimer(0.1f, FWidgetActiveTimerDelegate::CreateSP(this, &SReferenceViewer::TriggerZoomToFit));

	// Set the initial history data
	HistoryManager.AddHistoryData();

	TemporaryPathBeingEdited = NewGraphRootIdentifiers.Num() > 0 ? FText() : FText(LOCTEXT("NoAssetsFound", "No Assets Found"));
}

EActiveTimerReturnType SReferenceViewer::TriggerZoomToFit(double InCurrentTime, float InDeltaTime)
{
	if (GraphEditorPtr.IsValid())
	{
		GraphEditorPtr->ZoomToFit(false);
	}
	return EActiveTimerReturnType::Stop;
}

void SReferenceViewer::GetSelectedNodeAssetData(TArray<FAssetData>& OutAssetData) const
{
	TSet<UObject*> SelectedNodes = GraphEditorPtr->GetSelectedNodes();

	Algo::TransformIf(SelectedNodes, OutAssetData,
		[](UObject* AssetToCheck) -> bool
		{
			// might need to do more just doing what is done above for now
			if (AssetToCheck != nullptr && AssetToCheck->IsA<UEdGraphNode_Reference>())
			{
				return Cast<UEdGraphNode_Reference>(AssetToCheck)->GetAssetData().IsValid();
			}
			else
			{
				return false;
			}
		},
		[](UObject* NodeToGetDataFrom) -> FAssetData
		{
			return Cast<UEdGraphNode_Reference>(NodeToGetDataFrom)->GetAssetData();
		});
}

void SReferenceViewer::SetCurrentRegistrySource(const FAssetManagerEditorRegistrySource* RegistrySource)
{
	RebuildGraph();
}

void SReferenceViewer::OnNodeDoubleClicked(UEdGraphNode* Node)
{
	if (!GraphObj)
	{
		return;
	}

	const TArray<FAssetIdentifier> CurrentlyVisualizedAssets = GraphObj->GetCurrentGraphRootIdentifiers();

	bool bDependency = false;
	UEdGraphNode* ParentNode = nullptr;
	if (UEdGraphNode_Reference* ReferenceNode = Cast<UEdGraphNode_Reference>(Node))
	{
		// Overflow nodes have no identifiers
		if (ReferenceNode->IsOverflow())
		{
			if (ReferenceNode->GetReferencerPin()->LinkedTo.Num() > 0)
			{
				ParentNode = ReferenceNode->GetReferencerPin()->LinkedTo[0]->GetOwningNode();
			}
			else if (ReferenceNode->GetDependencyPin()->LinkedTo.Num() > 0)
			{
				bDependency = true;
				ParentNode = ReferenceNode->GetDependencyPin()->LinkedTo[0]->GetOwningNode();
			}
		}
	}

	bool bFoundOverflow = false;
	if (ParentNode)
	{
		if (UEdGraphNode_Reference* ParentReferenceNode = Cast<UEdGraphNode_Reference>(ParentNode))
		{
			FAssetIdentifier ParentID = ParentReferenceNode->GetIdentifier();
			GraphObj->ExpandNode(bDependency, ParentID);
			bFoundOverflow = true;
		}
	}

	if (!bFoundOverflow)
	{
		// turn off the find path tool if the user is wanting to center on another node
		Settings->SetFindPathEnabled(false);

		TSet<UObject*> Nodes;
		Nodes.Add(Node);
		ReCenterGraphOnNodes( Nodes );
	}

	OnReferenceViewerSelectionChanged().Broadcast(CurrentlyVisualizedAssets, GraphObj->GetCurrentGraphRootIdentifiers());
}

void SReferenceViewer::RebuildGraph()
{
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	if (AssetRegistryModule.Get().IsLoadingAssets())
	{
		// We are still discovering assets, listen for the completion delegate before building the graph
		if (!AssetRegistryModule.Get().OnFilesLoaded().IsBoundToObject(this))
		{
			AssetRegistryModule.Get().OnFilesLoaded().AddSP(this, &SReferenceViewer::OnInitialAssetRegistrySearchComplete);
		}
	}
	else
	{
		// All assets are already discovered, build the graph now, if we have one
		if (GraphObj)
		{
			GraphObj->RebuildGraph();
		}

		bDirtyResults = false;
		if (!AssetRefreshHandle.IsValid())
		{
			// Listen for updates
			AssetRefreshHandle = AssetRegistryModule.Get().OnAssetUpdated().AddSP(this, &SReferenceViewer::OnAssetRegistryChanged);
			AssetRegistryModule.Get().OnAssetAdded().AddSP(this, &SReferenceViewer::OnAssetRegistryChanged);
			AssetRegistryModule.Get().OnAssetRemoved().AddSP(this, &SReferenceViewer::OnAssetRegistryChanged);
		}

		IPluginManager& PluginManager = IPluginManager::Get();
		if (!PluginManager.OnPluginEdited().IsBoundToObject(this))
		{
			PluginManager.OnPluginEdited().AddSP(this, &SReferenceViewer::OnPluginEdited);
		}
	}
}

FActionMenuContent SReferenceViewer::OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed)
{
	// no context menu when not over a node
	return FActionMenuContent();
}

bool SReferenceViewer::IsBackEnabled() const
{
	return HistoryManager.CanGoBack();
}

bool SReferenceViewer::IsForwardEnabled() const
{
	return HistoryManager.CanGoForward();
}

void SReferenceViewer::BackClicked()
{
	// Storing assets visualized before using history to go back. Cannot get this as ref since the original array is about to change.
	const TArray<FAssetIdentifier> CurrentlyVisualizedAssets = GraphObj->GetCurrentGraphRootIdentifiers();

	Settings->SetFindPathEnabled(false);
	HistoryManager.GoBack();

	OnReferenceViewerSelectionChanged().Broadcast(CurrentlyVisualizedAssets, GraphObj->GetCurrentGraphRootIdentifiers());
}

void SReferenceViewer::ForwardClicked()
{
	// Storing assets visualized before using history to go forward. Cannot get this as ref since the original array is about to change.
	const TArray<FAssetIdentifier> CurrentlyVisualizedAssets = GraphObj->GetCurrentGraphRootIdentifiers();

	Settings->SetFindPathEnabled(false);
	HistoryManager.GoForward();

	OnReferenceViewerSelectionChanged().Broadcast(CurrentlyVisualizedAssets, GraphObj->GetCurrentGraphRootIdentifiers());
}

void SReferenceViewer::RefreshClicked()
{
	RebuildGraph();
	TriggerZoomToFit(0, 0);
	RegisterActiveTimer(0.1f, FWidgetActiveTimerDelegate::CreateSP(this, &SReferenceViewer::TriggerZoomToFit));
}

void SReferenceViewer::GraphNavigateHistoryBack()
{
	BackClicked();
}

void SReferenceViewer::GraphNavigateHistoryForward()
{
	ForwardClicked();
}

FText SReferenceViewer::GetHistoryBackTooltip() const
{
	const FReferenceViewerHistoryData* HistoryData = HistoryManager.GetBackHistoryData();
	if (HistoryData != nullptr)
	{
		return FText::Format( LOCTEXT("HistoryBackTooltip", "Back to {0}"), GetIdentifierSummaryText(HistoryData->Identifiers) );
	}
	return FText::GetEmpty();
}

FText SReferenceViewer::GetHistoryForwardTooltip() const
{
	const FReferenceViewerHistoryData* HistoryData = HistoryManager.GetForwardHistoryData();
	if (HistoryData != nullptr)
	{
		return FText::Format( LOCTEXT("HistoryForwardTooltip", "Forward to {0}"), GetIdentifierSummaryText(HistoryData->Identifiers) );
	}
	return FText::GetEmpty();
}

FText SReferenceViewer::GetAddressBarText() const
{
	if ( GraphObj )
	{
		if (TemporaryPathBeingEdited.IsEmpty())
		{
			return GetIdentifierSummaryText(GraphObj->GetCurrentGraphRootIdentifiers());
		}
		else
		{
			return TemporaryPathBeingEdited;
		}
	}

	return FText();
}

FText SReferenceViewer::GetIdentifierSummaryText(const TArray<FAssetIdentifier>& Identifiers) const
{
	if (Identifiers.Num() == 1)
	{
		return GetIdentifierText(Identifiers[0]);
	}
	else if (Identifiers.Num() > 1)
	{
		return FText::Format(
			LOCTEXT("AddressBarMultiplePackagesText", "{0} and {1} others"),
			GetIdentifierText(Identifiers[0]),
			FText::AsNumber(Identifiers.Num() - 1)
		);
	}
	else
	{
		return LOCTEXT("NoAssetFoundText", "No Assets Found");
	}
}

FText SReferenceViewer::GetIdentifierText(const FAssetIdentifier& Identifier) const
{
	if (bShowingContentVersePath && !Identifier.GetPrimaryAssetId().IsValid() && !Identifier.IsValue())
	{
		TMap<FName, FAssetData> Assets;
		UE::AssetRegistry::GetAssetForPackages({ Identifier.PackageName }, Assets);
		if (const FAssetData* Asset = Assets.Find(Identifier.PackageName))
		{
			UE::Core::FVersePath VersePath = Asset->GetVersePath();
			if (VersePath.IsValid())
			{
				return FText::FromString(MoveTemp(VersePath).ToString());
			}
		}
	}
	return FText::FromString(Identifier.ToString());
}

FText SReferenceViewer::GetStatusText() const
{
	FString DirtyPackages;
	if (GraphObj)
	{
		const TArray<FAssetIdentifier>& CurrentGraphRootPackageNames = GraphObj->GetCurrentGraphRootIdentifiers();
		
		for (const FAssetIdentifier& CurrentAsset : CurrentGraphRootPackageNames)
		{
			if (CurrentAsset.IsPackage())
			{
				FString PackageString = CurrentAsset.PackageName.ToString();
				UPackage* InMemoryPackage = FindPackage(nullptr, *PackageString);
				if (InMemoryPackage && InMemoryPackage->IsDirty())
				{
					DirtyPackages += FPackageName::GetShortName(*PackageString);

					// Break on first modified asset to avoid string going too long, the multi select case is fairly rare
					break;
				}
			}
		}
	}

	if (DirtyPackages.Len() > 0)
	{
		return FText::Format(LOCTEXT("ModifiedWarning", "Showing old saved references for edited asset {0}"), FText::FromString(DirtyPackages));
	}

	if (bDirtyResults)
	{
		return LOCTEXT("DirtyWarning", "Saved references changed, refresh for update");
	}

	return FText();
}

FText SReferenceViewer::GetCenteredStatusText() const
{
	if (GraphObj && GraphObj->Nodes.IsEmpty())
	{
		return LOCTEXT("NoAssets", "No Assets Found");
	}

	return FText();
}

EVisibility SReferenceViewer::GetCenteredStatusVisibility() const
{
	if (GraphObj && GraphObj->Nodes.IsEmpty())
	{
		return EVisibility::Visible;
	}

	return EVisibility::Collapsed;
}

void SReferenceViewer::OnAddressBarTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo)
{
	if (CommitInfo == ETextCommit::OnEnter)
	{
		if (!GraphObj)
		{
			return;
		}

		const TArray<FAssetIdentifier> CurrentlyVisualizedAssets = GraphObj->GetCurrentGraphRootIdentifiers();

		TArray<FAssetIdentifier> NewPaths;
		FAssetIdentifier NewPath = FAssetIdentifier::FromString(NewText.ToString());

		FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked< FAssetRegistryModule >(TEXT("AssetRegistry"));
		IAssetRegistry* AssetRegistry = &AssetRegistryModule.Get();

		// Using GetDependencies just to check if NewPath exists in the dependency tree. We do not actually care about the dependencies here
		TArray<FAssetIdentifier> UnusedDependencies;
		if (NewPath.IsValid() && AssetRegistry->GetDependencies(NewPath, UnusedDependencies))
		{
			NewPaths.Add(NewPath);
		}
		else if (bShowingContentVersePath)
		{
			// Syntactically the grammer for FAssetIdentifiers and FVersePaths overlap.
			// If the we couldn't find NewPath it may be because it is a Verse path, so check that next.
			UE::Core::FVersePath VersePath;
			if (UE::Core::FVersePath::TryMake(VersePath, NewText.ToString()))
			{
				FAssetData AssetData = FAssetToolsModule::GetModule().Get().FindAssetByVersePath(VersePath);
				if (AssetData.IsValid())
				{
					NewPath = FAssetIdentifier(AssetData.PackageName);
					if (AssetRegistry->GetDependencies(NewPath, UnusedDependencies))
					{
						NewPaths.Add(NewPath);
					}
				}
			}
		}

		if (CurrentlyVisualizedAssets != NewPaths)
		{
			SetGraphRootIdentifiers(NewPaths);

			OnReferenceViewerSelectionChanged().Broadcast(CurrentlyVisualizedAssets, GraphObj->GetCurrentGraphRootIdentifiers());
		}

		// Whether or not the path actually changed, the user has finished editing —
		// fall back to displaying the current graph root rather than stale typed text.
		TemporaryPathBeingEdited = FText();
	}
}

void SReferenceViewer::OnAddressBarTextChanged(const FText& NewText)
{
	TemporaryPathBeingEdited = NewText;
}

void SReferenceViewer::OnApplyHistoryData(const FReferenceViewerHistoryData& History)
{
	if ( GraphObj )
	{
		GraphObj->SetGraphRoot(History.Identifiers);
		UEdGraphNode_Reference* NewRootNode = GraphObj->RebuildGraph();
		
		if ( NewRootNode && ensure(GraphEditorPtr.IsValid()) )
		{
			GraphEditorPtr->SetNodeSelection(NewRootNode, true);
		}

		TemporaryPathBeingEdited = FText();
	}
}

void SReferenceViewer::OnUpdateHistoryData(FReferenceViewerHistoryData& HistoryData) const
{
	if ( GraphObj )
	{
		const TArray<FAssetIdentifier>& CurrentGraphRootIdentifiers = GraphObj->GetCurrentGraphRootIdentifiers();
		HistoryData.Identifiers = CurrentGraphRootIdentifiers;
	}
	else
	{
		HistoryData.Identifiers.Empty();
	}
}

void SReferenceViewer::OnUpdateFilterBar()
{
	bRebuildingFilters = true; 

	if (GraphObj)
	{
		const TSet<FTopLevelAssetPath> AllClasses = GraphObj->GetAssetTypes();
		if (Settings->AutoUpdateFilters())
		{
			FilterWidget->RemoveAllFilters();
			for (const FTopLevelAssetPath& AssetClassPath : AllClasses)
			{
				if (FilterWidget->DoesAssetTypeFilterExist(AssetClassPath))
				{
					FilterWidget->SetAssetTypeFilterCheckState(AssetClassPath, ECheckBoxState::Checked);
				}
				// If the current AssetClassPath does not have a filter in the filter bar, we walk through its ancestor classes to see if any of those have a filter
				else
				{
					TArray<FTopLevelAssetPath> AncestorClassNames;
					
					FAssetRegistryModule& AssetRegistryModule = FModuleManager::GetModuleChecked< FAssetRegistryModule >( TEXT("AssetRegistry") );
					AssetRegistryModule.Get().GetAncestorClassNames(AssetClassPath, AncestorClassNames);
					
					for (const FTopLevelAssetPath& AssetClassAncestor : AncestorClassNames)
					{
						if (FilterWidget->DoesAssetTypeFilterExist(AssetClassAncestor))
						{
							FilterWidget->SetAssetTypeFilterCheckState(AssetClassAncestor, ECheckBoxState::Checked);
							break;
						}
					}
				}
			}

			GraphObj->SetCurrentFilterCollection(FilterWidget->GetAllActiveFilters());
		}

		else
		{
			FilterWidget->LoadSettings();
			GraphObj->SetCurrentFilterCollection(FilterWidget->GetAllActiveFilters());
		}
	}

	bRebuildingFilters = false;
}

void SReferenceViewer::OnSearchDepthEnabledChanged( ECheckBoxState NewState )
{
	Settings->SetSearchDepthLimitEnabled(NewState == ECheckBoxState::Checked);
	RebuildGraph();
}

ECheckBoxState SReferenceViewer::IsSearchDepthEnabledChecked() const
{
	return Settings->IsSearchDepthLimited() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

int32 SReferenceViewer::GetSearchDependencyDepthCount() const
{
	return Settings->GetSearchDependencyDepthLimit();
}

int32 SReferenceViewer::GetSearchReferencerDepthCount() const
{
	return Settings->GetSearchReferencerDepthLimit();
}

void SReferenceViewer::OnSearchDependencyDepthCommitted(int32 NewValue)
{
	if (NewValue != Settings->GetSearchDependencyDepthLimit())
	{
		Settings->SetSearchDependencyDepthLimit(NewValue);
		RebuildGraph();
	}
}

void SReferenceViewer::OnSearchReferencerDepthCommitted(int32 NewValue)
{
	if (NewValue != Settings->GetSearchReferencerDepthLimit())
	{
		Settings->SetSearchReferencerDepthLimit(NewValue);
		RebuildGraph();
	}
}

void SReferenceViewer::OnEnableCollectionFilterChanged(ECheckBoxState NewState)
{
	const bool bNewValue = NewState == ECheckBoxState::Checked;
	const bool bCurrentValue = Settings->GetEnableCollectionFilter();
	if (bCurrentValue != bNewValue)
	{
		Settings->SetEnableCollectionFilter(NewState == ECheckBoxState::Checked);
		RebuildGraph();
	}
}

ECheckBoxState SReferenceViewer::IsEnableCollectionFilterChecked() const
{
	return Settings->GetEnableCollectionFilter() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

FText SReferenceViewer::GetCollectionComboButtonText() const
{
	ICollectionContainer* CurrentCollectionContainer = nullptr;
	FName CurrentCollectionName;
	GraphObj->GetCurrentCollectionFilter(CurrentCollectionContainer, CurrentCollectionName);

	return FText::FromName(CurrentCollectionName);
}

void SReferenceViewer::CollectionFilterAddMenuEntry(FMenuBuilder& MenuBuilder, const TSharedPtr<ICollectionContainer>& CollectionContainer, const FName& CollectionName)
{
	FExecuteAction ActionClicked = FExecuteAction::CreateLambda([this, CollectionContainer, CollectionName]()
	{
		// Make sure collection filtering is enabled now that the user clicked something in the menu.
		Settings->SetEnableCollectionFilter(true);

		ICollectionContainer* CurrentCollectionContainer = nullptr;
		FName CurrentCollectionName;
		GraphObj->GetCurrentCollectionFilter(CurrentCollectionContainer, CurrentCollectionName);

		// Update the filter and rebuild the graph if the filter changed.
		if (CurrentCollectionContainer != CollectionContainer.Get() || CurrentCollectionName != CollectionName)
		{
			GraphObj->SetCurrentCollectionFilter(CollectionContainer, CollectionName);
			RebuildGraph();
		}
	});

	FIsActionChecked ActionChecked = FIsActionChecked::CreateLambda([this, CollectionContainer, CollectionName]() -> bool
	{
		ICollectionContainer* CurrentCollectionContainer = nullptr;
		FName CurrentCollectionName;
		GraphObj->GetCurrentCollectionFilter(CurrentCollectionContainer, CurrentCollectionName);

		return CurrentCollectionContainer == CollectionContainer.Get() && CurrentCollectionName == CollectionName;
	});

	MenuBuilder.AddMenuEntry(
		FText::FromName(CollectionName),
		FText::FromName(CollectionName),
		FSlateIcon(),
		FUIAction(ActionClicked, FCanExecuteAction(), ActionChecked),
		NAME_None, //InExtensionHook
		EUserInterfaceActionType::RadioButton
		);
}

TSharedRef<SWidget> SReferenceViewer::BuildCollectionFilterMenu()
{
	FMenuBuilder MenuBuilder(true /* Pass true to close dropdown after selection. */, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("CollectionFilterSelectNone", "Select None"),
		LOCTEXT("CollectionFilterSelectNoCollection", "Select no collection."),
		FSlateIcon(),
		FExecuteAction::CreateLambda([this]()
		{
			// Make sure collection filtering is enabled.
			Settings->SetEnableCollectionFilter(true);

			GraphObj->SetCurrentCollectionFilter(nullptr, NAME_None);
			RebuildGraph();
		})
	);

	TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
	FCollectionManagerModule::GetModule().Get().GetVisibleCollectionContainers(CollectionContainers);

	if (CollectionContainers.Num() == 1)
	{
		MenuBuilder.AddSeparator();
	}

	TArray<FName> CollectionNames;
	TArray<FCollectionNameType> AllCollections;
	for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
	{
		CollectionNames.Reset();
		AllCollections.Reset();

		CollectionContainer->GetCollections(AllCollections);

		CollectionNames.Reserve(AllCollections.Num());
		for (const FCollectionNameType& Collection : AllCollections)
		{
			ECollectionStorageMode::Type StorageMode = ECollectionStorageMode::Static;
			CollectionContainer->GetCollectionStorageMode(Collection.Name, Collection.Type, StorageMode);

			if (StorageMode == ECollectionStorageMode::Static)
			{
				CollectionNames.AddUnique(Collection.Name);
			}
		}

		CollectionNames.Sort([](const FName& A, const FName& B) { return A.Compare(B) < 0; });

		if (CollectionContainers.Num() != 1)
		{
			MenuBuilder.BeginSection(NAME_None, CollectionContainer->GetCollectionSource()->GetTitle());
		}

		for (const FName& CollectionName : CollectionNames)
		{
			CollectionFilterAddMenuEntry(MenuBuilder, CollectionContainer, CollectionName);
		}

		if (CollectionContainers.Num() != 1)
		{
			MenuBuilder.EndSection();
		}
	}

	return MenuBuilder.MakeWidget();
}

void SReferenceViewer::OnEnablePluginFilterChanged(ECheckBoxState NewState)
{
	const bool bNewValue = NewState == ECheckBoxState::Checked;

	const bool bCurrentValue = Settings->GetEnablePluginFilter();
	if (bCurrentValue != bNewValue)
	{
		Settings->SetEnablePluginFilter(NewState == ECheckBoxState::Checked);
		RebuildGraph();
	}
}

ECheckBoxState SReferenceViewer::IsEnablePluginFilterChecked() const
{
	return Settings->GetEnablePluginFilter() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

void SReferenceViewer::PluginFilterAddMenuEntry(FMenuBuilder& MenuBuilder, const FName& PluginName, const FText& Label, const FText& ToolTip)
{
	FExecuteAction ActionClicked = FExecuteAction::CreateLambda([this, PluginName]()
	{
		TArray<FName> CurrentPluginFilter = GraphObj->GetCurrentPluginFilter();
		// We just got checked if we don't exist in the current plugin filter.
		const bool bNewChecked = !CurrentPluginFilter.Contains(PluginName);

		if (bNewChecked)
		{
			// Make sure plugin filtering is enabled now that something was checked.
			Settings->SetEnablePluginFilter(true);

			CurrentPluginFilter.AddUnique(PluginName);
		}
		else if (CurrentPluginFilter.Contains(PluginName))
		{
			CurrentPluginFilter.RemoveAll([PluginName](const FName& Name)
			{
				return Name == PluginName;
			});
		}

		GraphObj->SetCurrentPluginFilter(CurrentPluginFilter);
		RebuildGraph();
	});

	FIsActionChecked ActionChecked = FIsActionChecked::CreateLambda([this, PluginName]() -> bool
	{
		return GraphObj->GetCurrentPluginFilter().Contains(PluginName);
	});

	MenuBuilder.AddMenuEntry(
		Label,
		ToolTip,
		FSlateIcon(),
		FUIAction(ActionClicked, FCanExecuteAction(), ActionChecked),
		NAME_None, //InExtensionHook
		EUserInterfaceActionType::ToggleButton
		);
}

TSharedRef<SWidget> SReferenceViewer::BuildPluginFilterMenu()
{
	FMenuBuilder MenuBuilder(false, nullptr);

	MenuBuilder.AddMenuEntry(
		LOCTEXT("PluginFilterSelectAll", "Select All"),
		LOCTEXT("PluginFilterSelectAllPlugins", "Select all plugins."),
		FSlateIcon(),
		FExecuteAction::CreateLambda([this]()
		{
			// Make sure plugin filtering is enabled.
			Settings->SetEnablePluginFilter(true);

			GraphObj->SetCurrentPluginFilter(GraphObj->GetEncounteredPluginsAmongNodes());
			RebuildGraph();
		})
	);
	
	MenuBuilder.AddMenuEntry(
		LOCTEXT("PluginFilterSelectNone", "Select None"),
		LOCTEXT("PluginFilterSelectNoPlugins", "Select no plugins."),
		FSlateIcon(),
		FExecuteAction::CreateLambda([this]()
		{
			// Make sure plugin filtering is enabled.
			Settings->SetEnablePluginFilter(true);

			GraphObj->SetCurrentPluginFilter(TArray<FName>());
			RebuildGraph();
		})
	);

	TArray<FName> PluginNames = GraphObj->GetEncounteredPluginsAmongNodes();
	if (PluginNames.IsEmpty())
	{
		return MenuBuilder.MakeWidget();
	}

	// Create a map of plugin names to enabled plugins.
	TMap<FName, TSharedRef<IPlugin>> EnabledPlugins;
	{
		TArray<TSharedRef<IPlugin>> EnabledPluginsWithContent = IPluginManager::Get().GetEnabledPluginsWithContent();
		EnabledPlugins.Reserve(EnabledPluginsWithContent.Num());
		for (TSharedRef<IPlugin>& Plugin : EnabledPluginsWithContent)
		{
			const FName Name = FName(Plugin->GetName());
			EnabledPlugins.Add(Name, MoveTemp(Plugin));
		}
	}

	struct FPluginFilter
	{
		FPluginFilter(FName InPluginName, const TSharedRef<IPlugin>* InPlugin)
			: PluginName(InPluginName)
			, bIsPlugin(InPlugin != nullptr)
			, ToolTip(FText::FromName(InPluginName))
		{
			if (InPlugin != nullptr)
			{
				Label = FText::FromString((*InPlugin)->GetFriendlyName());
			}
			else
			{
				Label = ToolTip;
			}
		}

		FName PluginName;
		bool bIsPlugin;
		FText Label;
		FText ToolTip;
	};

	TArray<FPluginFilter> PluginFilters;
	PluginFilters.Reserve(PluginNames.Num());

	for (const FName& PluginName : PluginNames)
	{
		PluginFilters.Emplace(PluginName, EnabledPlugins.Find(PluginName));
	}
	
	// Sort non-real plugins (such as /Game and /Engine) first, then by display label.
	PluginFilters.Sort([](const FPluginFilter& A, const FPluginFilter& B)
	{
		if (A.bIsPlugin != B.bIsPlugin)
		{
			return !A.bIsPlugin;
		}
		return A.Label.CompareTo(B.Label) < 0;
	});

	MenuBuilder.AddSeparator();

	for (int32 Index = 0; Index < PluginFilters.Num(); ++Index)
	{
		const FPluginFilter& PluginFilter = PluginFilters[Index];

		// Add a separator between non-real plugins (such as /Game and /Engine) and actual plugins.
		if (Index > 0 && PluginFilters[Index - 1].bIsPlugin != PluginFilter.bIsPlugin)
		{
			MenuBuilder.AddSeparator();
		}

		PluginFilterAddMenuEntry(MenuBuilder, PluginFilter.PluginName, PluginFilter.Label, PluginFilter.ToolTip);
	}

	return MenuBuilder.MakeWidget();
}

FText SReferenceViewer::GetPluginComboButtonText() const
{
	const TArray<FName> CurrentPluginFilter = GraphObj->GetCurrentPluginFilter();

	if (CurrentPluginFilter.IsEmpty())
	{
		return LOCTEXT("PluginFilterNothingSelected", "None");
	}
	else if (CurrentPluginFilter.Num() == 1)
	{
		FNameBuilder NameBuilder;
		CurrentPluginFilter[0].AppendString(NameBuilder);
		TSharedPtr<IPlugin> Plugin = IPluginManager::Get().FindPlugin(NameBuilder);
		if (Plugin)
		{
			return FText::FromString(Plugin->GetFriendlyName());
		}
		return FText::FromName(CurrentPluginFilter[0]);
	}
	else
	{
		return LOCTEXT("PluginFilterMultipleSelected", "Multiple");
	}
}

void SReferenceViewer::OnShowSoftReferencesChanged()
{
	Settings->SetShowSoftReferencesEnabled( !Settings->IsShowSoftReferences() );
	RebuildGraph();
}

bool SReferenceViewer::IsShowSoftReferencesChecked() const
{
	return Settings->IsShowSoftReferences();
}

void SReferenceViewer::OnShowHardReferencesChanged()
{
	Settings->SetShowHardReferencesEnabled(!Settings->IsShowHardReferences());
	RebuildGraph();
}

bool SReferenceViewer::IsShowHardReferencesChecked() const
{
	return Settings->IsShowHardReferences();
}

void SReferenceViewer::OnShowFilteredPackagesOnlyChanged()
{
	Settings->SetShowFilteredPackagesOnlyEnabled(!Settings->IsShowFilteredPackagesOnly());
	UpdateIsPassingSearchFilterCallback();
}


bool SReferenceViewer::IsShowFilteredPackagesOnlyChecked() const
{
	return Settings->IsShowFilteredPackagesOnly();
}

void SReferenceViewer::UpdateIsPassingSearchFilterCallback()
{
	if (GraphObj)
	{
		UEdGraph_ReferenceViewer::FDoesAssetPassSearchFilterCallback DoesAssetPassSearchFilterCallback;
		if (Settings->IsShowFilteredPackagesOnly())
		{
			FString SearchString = SearchBox->GetText().ToString();
			TArray<FString> SearchWords;
			SearchString.ParseIntoArrayWS(SearchWords);
			if (SearchWords.Num() > 0)
			{
				DoesAssetPassSearchFilterCallback = [bShowingContentVersePath = bShowingContentVersePath, SearchWords](const FAssetIdentifier& InAssetIdentifier, const FAssetData& InAssetData)
				{
					return DoesAssetPassSearchTextFilter(InAssetIdentifier, InAssetData, bShowingContentVersePath, SearchWords);
				};
			}
		}
		GraphObj->SetDoesAssetPassSearchFilterCallback(DoesAssetPassSearchFilterCallback);
		GraphObj->RefilterGraph();
	}
}

void SReferenceViewer::OnCompactModeChanged()
{
	Settings->SetCompactModeEnabled(!Settings->IsCompactMode());
	RebuildGraph();
}

bool SReferenceViewer::IsCompactModeChecked() const
{
	return Settings->IsCompactMode();
}

void SReferenceViewer::OnShowExternalReferencersChanged()
{
	Settings->SetShowExternalReferencersEnabled(!Settings->IsShowExternalReferencers());
	RebuildGraph();
}

bool SReferenceViewer::IsShowExternalReferencersChecked() const
{
	return Settings->IsShowExternalReferencers();
}

void SReferenceViewer::OnShowDuplicatesChanged()
{
	Settings->SetShowDuplicatesEnabled(!Settings->IsShowDuplicates());
	if (GraphObj)
	{
		GraphObj->RefilterGraph();
	}
}

bool SReferenceViewer::IsShowDuplicatesChecked() const
{
	return Settings->GetFindPathEnabled() || Settings->IsShowDuplicates();
}

void SReferenceViewer::OnEditorOnlyReferenceFilterTypeChanged(EEditorOnlyReferenceFilterType Value)
{
	Settings->SetEditorOnlyReferenceFilterType(Value);
	if (GraphObj)
	{
		GraphObj->RebuildGraph();
	}
}

EEditorOnlyReferenceFilterType SReferenceViewer::GetEditorOnlyReferenceFilterType() const
{
	return Settings->GetEditorOnlyReferenceFilterType();
}

bool SReferenceViewer::GetManagementReferencesVisibility() const
{
	return bShowShowReferencesOptions;
}

void SReferenceViewer::OnShowManagementReferencesChanged()
{
	// This can take a few seconds if it isn't ready
	UAssetManager::Get().UpdateManagementDatabase();

	Settings->SetShowManagementReferencesEnabled(!Settings->IsShowManagementReferences());
	RebuildGraph();
}

bool SReferenceViewer::IsShowManagementReferencesChecked() const
{
	return Settings->IsShowManagementReferences();
}

void SReferenceViewer::OnShowSearchableNamesChanged()
{
	Settings->SetShowSearchableNames(!Settings->IsShowSearchableNames());
	RebuildGraph();
}

bool SReferenceViewer::IsShowSearchableNamesChecked() const
{
	return Settings->IsShowSearchableNames();
}

void SReferenceViewer::OnShowCodePackagesChanged()
{
	Settings->SetShowCodePackages(!Settings->IsShowCodePackages());
	RebuildGraph();
}

bool SReferenceViewer::IsShowCodePackagesChecked() const
{
	return Settings->IsShowCodePackages();
}

int32 SReferenceViewer::GetSearchBreadthCount() const
{
	return Settings->GetSearchBreadthLimit();
}

void SReferenceViewer::SetSearchBreadthCount(int32 InBreadthValue)
{
	if (!Settings)
	{
		return;
	}

	if (Settings->GetSearchBreadthLimit() != InBreadthValue)
	{
		Settings->SetSearchBreadthLimit(InBreadthValue);
	}
}

void SReferenceViewer::OnSearchBreadthChanged(int32 InBreadthValue)
{
	SetSearchBreadthCount(InBreadthValue);

	bNeedsGraphRefilter = true;
}

void SReferenceViewer::OnSearchBreadthCommited(int32 InBreadthValue, ETextCommit::Type InCommitType)
{
	SetSearchBreadthCount(InBreadthValue);

	bNeedsGraphRefilter = false;

	if (GraphObj)
	{
		GraphObj->RefilterGraph();
	}

	FSlateApplication::Get().SetKeyboardFocus(GraphEditorPtr, EFocusCause::SetDirectly);
}

void SReferenceViewer::RegisterActions()
{
	ReferenceViewerActions = MakeShareable(new FUICommandList);
	FAssetManagerEditorCommands::Register();

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ZoomToFit,
		FExecuteAction::CreateSP(this, &SReferenceViewer::ZoomToFit),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::CanZoomToFit));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ResolveReferencingProperties,
		FExecuteAction::CreateSP(this, &SReferenceViewer::ResolveReferencingProperties),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::CanResolveReferencingProperties));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().Find,
		FExecuteAction::CreateSP(this, &SReferenceViewer::OnFind));

	ReferenceViewerActions->MapAction(
		FGlobalEditorCommonCommands::Get().FindInContentBrowser,
		FExecuteAction::CreateSP(this, &SReferenceViewer::ShowSelectionInContentBrowser),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::HasAtLeastOnePackageNodeSelected));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().OpenSelectedInAssetEditor,
		FExecuteAction::CreateSP(this, &SReferenceViewer::OpenSelectedInAssetEditor),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::HasAtLeastOneRealNodeSelected));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ReCenterGraph,
		FExecuteAction::CreateSP(this, &SReferenceViewer::ReCenterGraph),
		FCanExecuteAction());

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().IncreaseReferencerSearchDepth,
		FExecuteAction::CreateLambda( [this] { OnSearchReferencerDepthCommitted( GetSearchReferencerDepthCount() + 1); } ),
		FCanExecuteAction());

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().DecreaseReferencerSearchDepth,
		FExecuteAction::CreateLambda( [this] { OnSearchReferencerDepthCommitted( GetSearchReferencerDepthCount() - 1); } ),
		FCanExecuteAction());

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().SetReferencerSearchDepth,
		FExecuteAction::CreateLambda( [this] { FSlateApplication::Get().SetKeyboardFocus(ReferencerCountBox, EFocusCause::SetDirectly); } ),
		FCanExecuteAction());

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().IncreaseDependencySearchDepth,
		FExecuteAction::CreateLambda( [this] { OnSearchDependencyDepthCommitted( GetSearchDependencyDepthCount() + 1); } ),
		FCanExecuteAction());

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().DecreaseDependencySearchDepth,
		FExecuteAction::CreateLambda( [this] { OnSearchDependencyDepthCommitted( GetSearchDependencyDepthCount() - 1); } ),
		FCanExecuteAction());

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().SetDependencySearchDepth,
		FExecuteAction::CreateLambda( [this] { FSlateApplication::Get().SetKeyboardFocus(DependencyCountBox, EFocusCause::SetDirectly); } ),
		FCanExecuteAction());

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().IncreaseBreadth,
		FExecuteAction::CreateLambda( [this] { SetSearchBreadthCount( GetSearchBreadthCount() + 1); } ),
		FCanExecuteAction());

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().DecreaseBreadth,
		FExecuteAction::CreateLambda( [this] { SetSearchBreadthCount( GetSearchBreadthCount() - 1); } ),
		FCanExecuteAction());

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().SetBreadth,
		FExecuteAction::CreateLambda( [this] { FSlateApplication::Get().SetKeyboardFocus(BreadthLimitBox, EFocusCause::SetDirectly); } ),
		FCanExecuteAction());

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ShowSoftReferences,
		FExecuteAction::CreateSP(this, &SReferenceViewer::OnShowSoftReferencesChanged),
		FCanExecuteAction(),	
		FIsActionChecked::CreateSP(this, &SReferenceViewer::IsShowSoftReferencesChecked),
		FIsActionButtonVisible::CreateLambda([this] { return bShowShowReferencesOptions; }));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ShowHardReferences,
		FExecuteAction::CreateSP(this, &SReferenceViewer::OnShowHardReferencesChanged),
		FCanExecuteAction(),	
		FIsActionChecked::CreateSP(this, &SReferenceViewer::IsShowHardReferencesChecked),
		FIsActionButtonVisible::CreateLambda([this] { return bShowShowReferencesOptions; }));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().EditorOnlyReferenceFilterTypeGame,
		FExecuteAction::CreateSPLambda(this, [this]() { OnEditorOnlyReferenceFilterTypeChanged(EEditorOnlyReferenceFilterType::Game); }),
		FCanExecuteAction(),	
		FIsActionChecked::CreateSPLambda(this, [this]() { return GetEditorOnlyReferenceFilterType() == EEditorOnlyReferenceFilterType::Game; }),
		FIsActionButtonVisible::CreateLambda([this] { return bShowShowReferencesOptions; }));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().EditorOnlyReferenceFilterTypePropagation,
		FExecuteAction::CreateSPLambda(this, [this]() { OnEditorOnlyReferenceFilterTypeChanged(EEditorOnlyReferenceFilterType::Propagation); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateSPLambda(this, [this]() { return GetEditorOnlyReferenceFilterType() == EEditorOnlyReferenceFilterType::Propagation; }),
		FIsActionButtonVisible::CreateLambda([this] { return bShowShowReferencesOptions; }));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().EditorOnlyReferenceFilterTypeEditorOnly,
		FExecuteAction::CreateSPLambda(this, [this]() { OnEditorOnlyReferenceFilterTypeChanged(EEditorOnlyReferenceFilterType::EditorOnly); }),
		FCanExecuteAction(),
		FIsActionChecked::CreateSPLambda(this, [this]() { return GetEditorOnlyReferenceFilterType() == EEditorOnlyReferenceFilterType::EditorOnly; }),
		FIsActionButtonVisible::CreateLambda([this] { return bShowShowReferencesOptions; }));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ShowManagementReferences,
		FExecuteAction::CreateSP(this, &SReferenceViewer::OnShowManagementReferencesChanged),
		FCanExecuteAction(),	
		FIsActionChecked::CreateSP(this, &SReferenceViewer::IsShowManagementReferencesChecked),
		FIsActionButtonVisible::CreateSP(this, &SReferenceViewer::GetManagementReferencesVisibility));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ShowNameReferences,
		FExecuteAction::CreateSP(this, &SReferenceViewer::OnShowSearchableNamesChanged),
		FCanExecuteAction(),	
		FIsActionChecked::CreateSP(this, &SReferenceViewer::IsShowSearchableNamesChecked),
		FIsActionButtonVisible::CreateLambda([this] { return bShowShowSearchableNames; }));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ShowCodePackages,
		FExecuteAction::CreateSP(this, &SReferenceViewer::OnShowCodePackagesChanged),
		FCanExecuteAction(),	
		FIsActionChecked::CreateSP(this, &SReferenceViewer::IsShowCodePackagesChecked),
		FIsActionButtonVisible::CreateLambda([this] { return bShowShowCodePackages; }));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ShowDuplicates,
		FExecuteAction::CreateSP(this, &SReferenceViewer::OnShowDuplicatesChanged),
		FCanExecuteAction::CreateLambda([this] { return !Settings->GetFindPathEnabled(); }),
		FIsActionChecked::CreateSP(this, &SReferenceViewer::IsShowDuplicatesChecked));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().CompactMode,
		FExecuteAction::CreateSP(this, &SReferenceViewer::OnCompactModeChanged),
		FCanExecuteAction(),	
		FIsActionChecked::CreateSP(this, &SReferenceViewer::IsCompactModeChecked),
		FIsActionButtonVisible::CreateLambda([this] { return bShowCompactMode; }));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ShowExternalReferencers,
		FExecuteAction::CreateSP(this, &SReferenceViewer::OnShowExternalReferencersChanged),
		FCanExecuteAction(),	
		FIsActionChecked::CreateSP(this, &SReferenceViewer::IsShowExternalReferencersChecked));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().FilterSearch,
		FExecuteAction::CreateSP(this, &SReferenceViewer::OnShowFilteredPackagesOnlyChanged),
		FCanExecuteAction(),	
		FIsActionChecked::CreateSP(this, &SReferenceViewer::IsShowFilteredPackagesOnlyChecked),
		FIsActionButtonVisible::CreateLambda([this] { return bShowShowFilteredPackagesOnly; }) );

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().CopyReferencedObjects,
		FExecuteAction::CreateSP(this, &SReferenceViewer::CopyReferencedObjects),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::HasAtLeastOnePackageNodeSelected));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().CopyReferencingObjects,
		FExecuteAction::CreateSP(this, &SReferenceViewer::CopyReferencingObjects),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::HasAtLeastOnePackageNodeSelected));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ShowReferencedObjects,
		FExecuteAction::CreateSP(this, &SReferenceViewer::ShowReferencedObjects),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::HasAtLeastOnePackageNodeSelected));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ShowReferencingObjects,
		FExecuteAction::CreateSP(this, &SReferenceViewer::ShowReferencingObjects),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::HasAtLeastOnePackageNodeSelected));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().MakeLocalCollectionWithReferencers,
		FExecuteAction::CreateSP(this, &SReferenceViewer::MakeCollectionWithReferencersOrDependencies, ECollectionShareType::CST_Local, true),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::CanMakeCollectionWithReferencersOrDependencies, ECollectionShareType::CST_Local));
	
	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().MakePrivateCollectionWithReferencers,
		FExecuteAction::CreateSP(this, &SReferenceViewer::MakeCollectionWithReferencersOrDependencies, ECollectionShareType::CST_Private, true),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::CanMakeCollectionWithReferencersOrDependencies, ECollectionShareType::CST_Private));
	
	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().MakeSharedCollectionWithReferencers,
		FExecuteAction::CreateSP(this, &SReferenceViewer::MakeCollectionWithReferencersOrDependencies, ECollectionShareType::CST_Shared, true),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::CanMakeCollectionWithReferencersOrDependencies, ECollectionShareType::CST_Shared));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().MakeLocalCollectionWithDependencies,
		FExecuteAction::CreateSP(this, &SReferenceViewer::MakeCollectionWithReferencersOrDependencies, ECollectionShareType::CST_Local, false),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::CanMakeCollectionWithReferencersOrDependencies, ECollectionShareType::CST_Local));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().MakePrivateCollectionWithDependencies,
		FExecuteAction::CreateSP(this, &SReferenceViewer::MakeCollectionWithReferencersOrDependencies, ECollectionShareType::CST_Private, false),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::CanMakeCollectionWithReferencersOrDependencies, ECollectionShareType::CST_Private));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().MakeSharedCollectionWithDependencies,
		FExecuteAction::CreateSP(this, &SReferenceViewer::MakeCollectionWithReferencersOrDependencies, ECollectionShareType::CST_Shared, false),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::CanMakeCollectionWithReferencersOrDependencies, ECollectionShareType::CST_Shared));
	
	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ShowReferenceTree,
		FExecuteAction::CreateSP(this, &SReferenceViewer::ShowReferenceTree),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::HasExactlyOnePackageNodeSelected));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ViewSizeMap,
		FExecuteAction::CreateSP(this, &SReferenceViewer::ViewSizeMap),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::HasAtLeastOneRealNodeSelected));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ViewAssetAudit,
		FExecuteAction::CreateSP(this, &SReferenceViewer::ViewAssetAudit),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::HasAtLeastOneRealNodeSelected));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().ShowCommentPath,
		FExecuteAction::CreateLambda([this] { 
			Settings->SetShowPathEnabled(!Settings->IsShowPath());
			if (GraphObj)
			{
				GraphObj->RefilterGraph();
			}
		}),
		FCanExecuteAction(),	
		FIsActionChecked::CreateLambda([this] {return Settings->IsShowPath();}));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().Filters,
		FExecuteAction::CreateLambda([this] { 
			Settings->SetFiltersEnabled(!Settings->GetFiltersEnabled());
			if (GraphObj)
			{
				GraphObj->RefilterGraph();
			}
		}),
		FCanExecuteAction::CreateLambda([this] { return !Settings->GetFindPathEnabled(); }),
		FIsActionChecked::CreateLambda([this] {return !Settings->GetFindPathEnabled() && Settings->GetFiltersEnabled();}));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().AutoFilters,
		FExecuteAction::CreateLambda([this] { 
			Settings->SetAutoUpdateFilters(!Settings->AutoUpdateFilters());
			if (GraphObj)
			{
				OnUpdateFilterBar();
				GraphObj->RefilterGraph();
			}
		}),
		FCanExecuteAction::CreateLambda([this] {return !Settings->GetFindPathEnabled() && Settings->GetFiltersEnabled();}),
		FIsActionChecked::CreateLambda([this] { return !Settings->GetFindPathEnabled() && Settings->AutoUpdateFilters();}));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().FindPath,
		FExecuteAction::CreateLambda([this] 
		{
			bool bWasEnabled = Settings->GetFindPathEnabled();
			Settings->SetFindPathEnabled(!bWasEnabled);

			if (!bWasEnabled && !FindPathAssetId.IsValid())
			{
				FindPathAssetPicker->SetIsOpen(true);
			}
			
			GraphObj->RebuildGraph();

			RegisterActiveTimer(0.1f, FWidgetActiveTimerDelegate::CreateSP(this, &SReferenceViewer::TriggerZoomToFit));

		}),
		FCanExecuteAction::CreateLambda([this] { return GraphObj ? GraphObj->GetCurrentGraphRootIdentifiers().Num() == 1 : false; }),
		FIsActionChecked::CreateLambda([this] { return Settings->GetFindPathEnabled();}));

	ReferenceViewerActions->MapAction(
		FAssetManagerEditorCommands::Get().CopyPaths,
		FExecuteAction::CreateLambda([this] {
				FString Result;
				// Build up a list of selected assets from the graph selection set
				TSet<UObject*> SelectedNodes = GraphEditorPtr->GetSelectedNodes();
				for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
				{
					if (UEdGraphNode_Reference* ReferenceNode = Cast<UEdGraphNode_Reference>(*It))
					{
						if (ReferenceNode->GetAssetData().IsValid())
						{
							if (!Result.IsEmpty())
							{
								Result += LINE_TERMINATOR;
							}

							Result += ReferenceNode->GetAssetData().PackageName.ToString();
						}
					}
				}

				if (Result.Len())
				{
					FPlatformApplicationMisc::ClipboardCopy(*Result);
				}
		}),
		FCanExecuteAction::CreateSP(this, &SReferenceViewer::HasAtLeastOneRealNodeSelected));

	IAssetManagerEditorModule::Get().AppendRegisteredCommandsToCommandList(ReferenceViewerActions.ToSharedRef());
}

void SReferenceViewer::ShowSelectionInContentBrowser()
{
	TArray<FAssetData> AssetList;

	// Build up a list of selected assets from the graph selection set
	TSet<UObject*> SelectedNodes = GraphEditorPtr->GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
	{
		if (UEdGraphNode_Reference* ReferenceNode = Cast<UEdGraphNode_Reference>(*It))
		{
			if (ReferenceNode->GetAssetData().IsValid())
			{
				AssetList.Add(ReferenceNode->GetAssetData());
			}
		}
	}

	if (AssetList.Num() > 0)
	{
		GEditor->SyncBrowserToObjects(AssetList);
	}
}

void SReferenceViewer::OpenSelectedInAssetEditor()
{
	TArray<FAssetIdentifier> IdentifiersToEdit;
	TSet<UObject*> SelectedNodes = GraphEditorPtr->GetSelectedNodes();
	for (FGraphPanelSelectionSet::TConstIterator It(SelectedNodes); It; ++It)
	{
		if (UEdGraphNode_Reference* ReferenceNode = Cast<UEdGraphNode_Reference>(*It))
		{
			if (!ReferenceNode->IsCollapsed())
			{
				ReferenceNode->GetAllIdentifiers(IdentifiersToEdit);
			}
		}
	}

	// This will handle packages as well as searchable names if other systems register
	FEditorDelegates::OnEditAssetIdentifiers.Broadcast(IdentifiersToEdit);
}

void SReferenceViewer::ReCenterGraph()
{
	ReCenterGraphOnNodes( GraphEditorPtr->GetSelectedNodes() );
}

FString SReferenceViewer::GetObjectsList(EObjectsListType ObjectsListType) const
{
	FString ObjectsList;

	TSet<FName> AllSelectedPackageNames;
	GetPackageNamesFromSelectedNodes(AllSelectedPackageNames);

	if (AllSelectedPackageNames.Num() > 0)
	{
		FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));

		TArray<FName> HardPackageNames;
		TArray<FName> SoftPackageNames;
		TArray<FName> AllPackageNames;
		TMap<FName, FAssetData> Assets;

		for (const FName& SelectedPackageName : AllSelectedPackageNames)
		{
			HardPackageNames.Reset();
			SoftPackageNames.Reset();
			AllPackageNames.Reset();
			Assets.Reset();

			const TCHAR* ObjectsListName = nullptr;
			switch (ObjectsListType)
			{
			case SReferenceViewer::EObjectsListType::Referenced:
				ObjectsListName = TEXT("Dependencies");

				AssetRegistryModule.Get().GetDependencies(SelectedPackageName, HardPackageNames, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);
				AssetRegistryModule.Get().GetDependencies(SelectedPackageName, SoftPackageNames, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Soft);
				break;
			case SReferenceViewer::EObjectsListType::Referencing:
				ObjectsListName = TEXT("Referencers");

				AssetRegistryModule.Get().GetReferencers(SelectedPackageName, HardPackageNames, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Hard);
				AssetRegistryModule.Get().GetReferencers(SelectedPackageName, SoftPackageNames, UE::AssetRegistry::EDependencyCategory::Package, UE::AssetRegistry::EDependencyQuery::Soft);
				break;
			default:
				checkNoEntry();
				return {};
			}

			AllPackageNames.Add(SelectedPackageName);
			AllPackageNames.Append(HardPackageNames);
			AllPackageNames.Append(SoftPackageNames);

			AllPackageNames.Sort(FNameFastLess());
			AllPackageNames.SetNum(Algo::Unique(AllPackageNames));
			
			UE::AssetRegistry::GetAssetForPackages(AllPackageNames, Assets);

			auto AppendPackageNames = [this, &ObjectsList, &Assets](const TCHAR* Title, const TArray<FName>& PackageNames)
			{
				if (PackageNames.IsEmpty())
				{
					return;
				}

				ObjectsList += LINE_TERMINATOR;
				ObjectsList += FString::Printf(TEXT("  [%s]"), Title);

				for (const FName& PackageName : PackageNames)
				{
					ObjectsList += LINE_TERMINATOR;
					ObjectsList += TEXT("    ");
					if (const FAssetData* Asset = Assets.Find(PackageName))
					{
						if (bShowingContentVersePath)
						{
							UE::Core::FVersePath VersePath = Asset->GetVersePath();
							if (VersePath.IsValid())
							{
								ObjectsList += VersePath.ToString();
								continue;
							}
						}

						ObjectsList += Asset->GetObjectPathString();
					}
					else
					{
						const FString PackageString = PackageName.ToString();
						ObjectsList += FString::Printf(TEXT("%s.%s"), *PackageString, *FPackageName::GetLongPackageAssetName(PackageString));
					}
				}
			};

			if (!ObjectsList.IsEmpty())
			{
				ObjectsList += LINE_TERMINATOR;
			}

			FString SelectedPackageString;
			{
				UE::Core::FVersePath VersePath;

				if (bShowingContentVersePath)
				{
					if (const FAssetData* Asset = Assets.Find(SelectedPackageName))
					{
						VersePath = Asset->GetVersePath();
					}
				}

				if (VersePath.IsValid())
				{
					SelectedPackageString = MoveTemp(VersePath).ToString();
				}
				else
				{
					SelectedPackageString = SelectedPackageName.ToString();
				}
			}

			ObjectsList += FString::Printf(TEXT("[%s - %s]"), *SelectedPackageString, ObjectsListName);
			AppendPackageNames(TEXT("HARD"), HardPackageNames);
			AppendPackageNames(TEXT("SOFT"), SoftPackageNames);
		}
	}

	return ObjectsList;
}

void SReferenceViewer::CopyReferencedObjects()
{
	const FString ReferencedObjectsList = GetObjectsList(EObjectsListType::Referenced);
	FPlatformApplicationMisc::ClipboardCopy(*ReferencedObjectsList);
}

void SReferenceViewer::CopyReferencingObjects()
{
	const FString ReferencingObjectsList = GetObjectsList(EObjectsListType::Referencing);
	FPlatformApplicationMisc::ClipboardCopy(*ReferencingObjectsList);
}

void SReferenceViewer::ShowReferencedObjects()
{
	const FString ReferencedObjectsList = GetObjectsList(EObjectsListType::Referenced);
	SGenericDialogWidget::OpenDialog(LOCTEXT("ReferencedObjectsDlgTitle", "Referenced Objects"), SNew(STextBlock).Text(FText::FromString(ReferencedObjectsList)));
}

void SReferenceViewer::ShowReferencingObjects()
{	
	const FString ReferencingObjectsList = GetObjectsList(EObjectsListType::Referencing);
	SGenericDialogWidget::OpenDialog(LOCTEXT("ReferencingObjectsDlgTitle", "Referencing Objects"), SNew(STextBlock).Text(FText::FromString(ReferencingObjectsList)));
}

bool SReferenceViewer::CanMakeCollectionWithReferencersOrDependencies(ECollectionShareType::Type ShareType) const
{
	return CanMakeCollectionWithReferencersOrDependencies(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(), ShareType);
}

void SReferenceViewer::MakeCollectionWithReferencersOrDependencies(ECollectionShareType::Type ShareType, bool bReferencers)
{
	MakeCollectionWithReferencersOrDependencies(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(), ShareType, bReferencers);
}

bool SReferenceViewer::CanMakeCollectionWithReferencersOrDependencies(TSharedPtr<ICollectionContainer> CollectionContainer, ECollectionShareType::Type ShareType) const
{
	if (!ensure(CollectionContainer))
	{
		return false;
	}

	return !CollectionContainer->IsReadOnly(ShareType) && HasExactlyOnePackageNodeSelected();
}

void SReferenceViewer::MakeCollectionWithReferencersOrDependencies(TSharedPtr<ICollectionContainer> CollectionContainer, ECollectionShareType::Type ShareType, bool bReferencers)
{
	if (!ensure(CollectionContainer))
	{
		return;
	}

	TSet<FName> AllSelectedPackageNames;
	GetPackageNamesFromSelectedNodes(AllSelectedPackageNames);

	if (AllSelectedPackageNames.Num() > 0)
	{
		if (ensure(ShareType != ECollectionShareType::CST_All))
		{
			FText CollectionNameAsText;
			FString FirstAssetName = FPackageName::GetLongPackageAssetName(AllSelectedPackageNames.Array()[0].ToString());
			if (bReferencers)
			{
				if (AllSelectedPackageNames.Num() > 1)
				{
					CollectionNameAsText = FText::Format(LOCTEXT("ReferencersForMultipleAssetNames", "{0}AndOthers_Referencers"), FText::FromString(FirstAssetName));
				}
				else
				{
					CollectionNameAsText = FText::Format(LOCTEXT("ReferencersForSingleAsset", "{0}_Referencers"), FText::FromString(FirstAssetName));
				}
			}
			else
			{
				if (AllSelectedPackageNames.Num() > 1)
				{
					CollectionNameAsText = FText::Format(LOCTEXT("DependenciesForMultipleAssetNames", "{0}AndOthers_Dependencies"), FText::FromString(FirstAssetName));
				}
				else
				{
					CollectionNameAsText = FText::Format(LOCTEXT("DependenciesForSingleAsset", "{0}_Dependencies"), FText::FromString(FirstAssetName));
				}
			}

			FName CollectionName;
			CollectionContainer->CreateUniqueCollectionName(*CollectionNameAsText.ToString(), ShareType, CollectionName);

			FText ResultsMessage;
			
			FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
			TArray<FName> PackageNamesToAddToCollection;
			if (bReferencers)
			{
				for (FName SelectedPackage : AllSelectedPackageNames)
				{
					AssetRegistryModule.Get().GetReferencers(SelectedPackage, PackageNamesToAddToCollection);
				}
			}
			else
			{
				for (FName SelectedPackage : AllSelectedPackageNames)
				{
					AssetRegistryModule.Get().GetDependencies(SelectedPackage, PackageNamesToAddToCollection);
				}
			}

			TSet<FName> PackageNameSet;
			for (FName PackageToAdd : PackageNamesToAddToCollection)
			{
				if (!AllSelectedPackageNames.Contains(PackageToAdd))
				{
					PackageNameSet.Add(PackageToAdd);
				}
			}

			IAssetManagerEditorModule::Get().WriteCollection(*CollectionContainer, CollectionName, ShareType, PackageNameSet.Array(), true);
		}
	}
}

void SReferenceViewer::ShowReferenceTree()
{
	UObject* SelectedObject = GetObjectFromSingleSelectedNode();

	if ( SelectedObject )
	{
		bool bObjectWasSelected = false;
		for (FSelectionIterator It(*GEditor->GetSelectedObjects()) ; It; ++It)
		{
			if ( (*It) == SelectedObject )
			{
				GEditor->GetSelectedObjects()->Deselect( SelectedObject );
				bObjectWasSelected = true;
			}
		}

		ObjectTools::ShowReferenceGraph( SelectedObject );

		if ( bObjectWasSelected )
		{
			GEditor->GetSelectedObjects()->Select( SelectedObject );
		}
	}
}

void SReferenceViewer::ViewSizeMap()
{
	TArray<FAssetIdentifier> AssetIdentifiers;
	TSet<UObject*> SelectedNodes = GraphEditorPtr->GetSelectedNodes();
	for (UObject* Node : SelectedNodes)
	{
		UEdGraphNode_Reference* ReferenceNode = Cast<UEdGraphNode_Reference>(Node);
		if (ReferenceNode)
		{
			ReferenceNode->GetAllIdentifiers(AssetIdentifiers);
		}
	}

	if (AssetIdentifiers.Num() > 0)
	{
		IAssetManagerEditorModule::Get().OpenSizeMapUI(AssetIdentifiers);
	}
}

void SReferenceViewer::ViewAssetAudit()
{
	TSet<FName> SelectedAssetPackageNames;
	GetPackageNamesFromSelectedNodes(SelectedAssetPackageNames);

	if (SelectedAssetPackageNames.Num() > 0)
	{
		IAssetManagerEditorModule::Get().OpenAssetAuditUI(SelectedAssetPackageNames.Array());
	}
}

void SReferenceViewer::ReCenterGraphOnNodes(const TSet<UObject*>& Nodes)
{
	TArray<FAssetIdentifier> NewGraphRootNames;
	FIntPoint TotalNodePos(ForceInitToZero);
	for ( auto NodeIt = Nodes.CreateConstIterator(); NodeIt; ++NodeIt )
	{
		UEdGraphNode_Reference* ReferenceNode = Cast<UEdGraphNode_Reference>(*NodeIt);
		if ( ReferenceNode )
		{
			ReferenceNode->GetAllIdentifiers(NewGraphRootNames);
			TotalNodePos.X += ReferenceNode->NodePosX;
			TotalNodePos.Y += ReferenceNode->NodePosY;
		}
	}

	if ( NewGraphRootNames.Num() > 0 )
	{
		const FIntPoint AverageNodePos = TotalNodePos / NewGraphRootNames.Num();
		GraphObj->SetGraphRoot(NewGraphRootNames, AverageNodePos);
		UEdGraphNode_Reference* NewRootNode = GraphObj->RebuildGraph();

		if ( NewRootNode && ensure(GraphEditorPtr.IsValid()) )
		{
			GraphEditorPtr->ClearSelectionSet();
			GraphEditorPtr->SetNodeSelection(NewRootNode, true);
		}

		// The graph root has changed; drop any in-progress address bar edit so the
		// displayed path reflects the new root instead of stale typed text.
		TemporaryPathBeingEdited = FText();

		// Set the initial history data
		HistoryManager.AddHistoryData();
	}
}

UObject* SReferenceViewer::GetObjectFromSingleSelectedNode() const
{
	UObject* ReturnObject = nullptr;

	TSet<UObject*> SelectedNodes = GraphEditorPtr->GetSelectedNodes();
	if ( ensure(SelectedNodes.Num()) == 1 )
	{
		UEdGraphNode_Reference* ReferenceNode = Cast<UEdGraphNode_Reference>(SelectedNodes.Array()[0]);
		if ( ReferenceNode )
		{
			const FAssetData& AssetData = ReferenceNode->GetAssetData();
			if (AssetData.IsAssetLoaded())
			{
				ReturnObject = AssetData.GetAsset();
			}
			else
			{
				FScopedSlowTask SlowTask(0, LOCTEXT("LoadingSelectedObject", "Loading selection..."));
				SlowTask.MakeDialog();
				ReturnObject = AssetData.GetAsset();
			}
		}
	}

	return ReturnObject;
}

void SReferenceViewer::GetPackageNamesFromSelectedNodes(TSet<FName>& OutNames) const
{
	TSet<UObject*> SelectedNodes = GraphEditorPtr->GetSelectedNodes();
	for (UObject* Node : SelectedNodes)
	{
		UEdGraphNode_Reference* ReferenceNode = Cast<UEdGraphNode_Reference>(Node);
		if (ReferenceNode)
		{
			TArray<FName> NodePackageNames;
			ReferenceNode->GetAllPackageNames(NodePackageNames);
			OutNames.Append(NodePackageNames);
		}
	}
}

bool SReferenceViewer::HasExactlyOneNodeSelected() const
{
	if ( GraphEditorPtr.IsValid() )
	{
		return GraphEditorPtr->GetSelectedNodes().Num() == 1;
	}
	
	return false;
}

bool SReferenceViewer::HasExactlyOnePackageNodeSelected() const
{
	if (GraphEditorPtr.IsValid())
	{
		TSet<UObject*> SelectedNodes = GraphEditorPtr->GetSelectedNodes();
		if (SelectedNodes.Num() != 1)
		{
			return false;
		}

		UObject* Node = *SelectedNodes.begin();
		UEdGraphNode_Reference* ReferenceNode = Cast<UEdGraphNode_Reference>(Node);
		if (ReferenceNode)
		{
			if (ReferenceNode->IsPackage())
			{
				return true;
			}
		}
		return false;
	}

	return false;
}

bool SReferenceViewer::HasAtLeastOnePackageNodeSelected() const
{
	if ( GraphEditorPtr.IsValid() )
	{
		TSet<UObject*> SelectedNodes = GraphEditorPtr->GetSelectedNodes();
		for (UObject* Node : SelectedNodes)
		{
			UEdGraphNode_Reference* ReferenceNode = Cast<UEdGraphNode_Reference>(Node);
			if (ReferenceNode)
			{
				if (ReferenceNode->IsPackage())
				{
					return true;
				}
			}
		}
	}
	
	return false;
}

bool SReferenceViewer::HasAtLeastOneRealNodeSelected() const
{
	if (GraphEditorPtr.IsValid())
	{
		TSet<UObject*> SelectedNodes = GraphEditorPtr->GetSelectedNodes();
		for (UObject* Node : SelectedNodes)
		{
			UEdGraphNode_Reference* ReferenceNode = Cast<UEdGraphNode_Reference>(Node);
			if (ReferenceNode)
			{
				if (!ReferenceNode->IsCollapsed())
				{
					return true;
				}
			}
		}
	}

	return false;
}

void SReferenceViewer::OnAssetRegistryChanged(const FAssetData& AssetData)
{
	// We don't do more specific checking because that data is not exposed, and it wouldn't handle newly added references anyway
	bDirtyResults = true;

	// Make sure referenced properties node are displaying updated information
	bNeedsReferencedPropertiesUpdate = true;
}

void SReferenceViewer::OnInitialAssetRegistrySearchComplete()
{
	if ( GraphObj )
	{
		GraphObj->RebuildGraph();
	}
}

void SReferenceViewer::OnPluginEdited(IPlugin& InPlugin)
{
	// The plugin's Verse path may have changed.  Recompute all Verse paths.
	if (GraphObj)
	{
		GraphObj->UpdatePaths();
	}
}

void SReferenceViewer::ZoomToFit()
{
	if (GraphEditorPtr.IsValid())
	{
		GraphEditorPtr->ZoomToFit(true);
	}
}

bool SReferenceViewer::CanZoomToFit() const
{
	if (GraphEditorPtr.IsValid())
	{
		return true;
	}

	return false;
}

void SReferenceViewer::OnFind()
{
	FSlateApplication::Get().SetKeyboardFocus(SearchBox, EFocusCause::SetDirectly);
}

void SReferenceViewer::ResolveReferencingProperties() const
{
	if (!GraphEditorPtr)
	{
		return;
	}

	// Retrieve Object from the specified node. Will load the asset if needed.
	auto GetObjectFromNode([](const UEdGraphNode_Reference* InNode)
	{
		UObject* ReturnObject;
		if (InNode)
		{
			const FAssetData& AssetData = InNode->GetAssetData();
			if (AssetData.IsAssetLoaded())
			{
				ReturnObject = AssetData.GetAsset();
			}
			else
			{
				FScopedSlowTask SlowTask(0, LOCTEXT("LoadingSelectedObject", "Loading selection..."));
				SlowTask.MakeDialog();
				ReturnObject = AssetData.GetAsset();
			}
		}
		else
		{
			ReturnObject = nullptr;
		}

		return ReturnObject;
	});

	TSet<UObject*> SelectedNodesAsObjects = GraphEditorPtr->GetSelectedNodes();
	if (ensure(!SelectedNodesAsObjects.IsEmpty()))
	{
		TSet<UEdGraphNode_Reference*> SelectedNodes;
		TSet<FAssetData> UnloadedAssetsData;

		// Retrieve current Reference Nodes, and keep track of those which need to be loaded
		for (UObject* SelectedNode : SelectedNodesAsObjects.Array())
		{
			UEdGraphNode_Reference* const ReferencedNode = Cast<UEdGraphNode_Reference>(SelectedNode);
			if (!ReferencedNode)
			{
				continue;
			}

			SelectedNodes.Add(ReferencedNode);

			// Look for referenced note asset, and check if it's loaded
			const FAssetData& AssetData = ReferencedNode->GetAssetData();
			if (!AssetData.IsAssetLoaded())
			{
				UnloadedAssetsData.Add(AssetData);
			}

			// Cycle all referencing nodes, and check if they're already loaded
			if (const UEdGraphPin* const ReferencerPin = ReferencedNode->GetReferencerPin())
			{
				for (const UEdGraphPin* const ReferencedPin : ReferencerPin->LinkedTo)
				{
					if (ReferencedPin)
					{
						if (const UEdGraphNode_Reference* const ReferencingNode =
								Cast<UEdGraphNode_Reference>(ReferencedPin->GetOwningNode()))
						{
							const FAssetData& ReferencerAssetData = ReferencingNode->GetAssetData();
							if (!ReferencerAssetData.IsAssetLoaded())
							{
								UnloadedAssetsData.Add(ReferencerAssetData);
							}
						}
					}
				}
			}
		}

		// If assets need to be loaded in order to resolve properties, let the user know
		if (!UnloadedAssetsData.IsEmpty())
		{
			const EAppReturnType::Type Ret = ShowAssetsNeedsToLoadMessage(UnloadedAssetsData, bShowingContentVersePath);
			if (Ret == EAppReturnType::Cancel)
			{
				return;
			}
		}

		FScopedSlowTask MainResolveTask(
			SelectedNodes.Num(),
			LOCTEXT("ReferencingProperties_ResolveTaskDialog", "Resolving Referencing Properties for selected nodes...")
		);
		MainResolveTask.MakeDialog(true);
		bool bIsCanceled = false;

		for (UEdGraphNode_Reference* ReferencedNode : SelectedNodes.Array())
		{
			if (MainResolveTask.ShouldCancel())
			{
				bIsCanceled = true;
			}

			if (bIsCanceled || !ReferencedNode)
			{
				break;
			}

			MainResolveTask.EnterProgressFrame(
				1.0f,
				FText::Format(
					LOCTEXT("ReferencingProperties_ResolveTaskDialogDetail", "Resolving Referencing Properties for {0}"),
					FText::FromName(ReferencedNode->GetAssetData().AssetName)
				)
			);

			UObject* const ReferencedObject = GetObjectFromNode(ReferencedNode);
			const UEdGraphPin* const ReferencerPin = ReferencedNode->GetReferencerPin();

			if (!ReferencerPin || !ReferencedObject)
			{
				continue;
			}

			const TArray<UEdGraphPin*> ReferencingPins = ReferencerPin->LinkedTo;
			if (!ReferencingPins.IsEmpty())
			{
				TArray<FReferencingPropertyDescription> ReferencingProperties;
				for (const UEdGraphPin* const ReferencedPin : ReferencingPins)
				{
					if (!ReferencedPin)
					{
						continue;
					}

					UEdGraphNode_Reference* const ReferencingNode =
						Cast<UEdGraphNode_Reference>(ReferencedPin->GetOwningNode());
					if (!ReferencingNode)
					{
						continue;
					}

					if (MainResolveTask.ShouldCancel())
					{
						bIsCanceled = true;
						break;
					}

					UObject* ReferencingObject = GetObjectFromNode(ReferencingNode);
					if (!ReferencingObject)
					{
						continue;
					}

					TArray<FReferencingPropertyDescription> ReferencingPropertiesArray =
						GraphObj->RetrieveReferencingProperties(ReferencingObject, ReferencedObject);

					GraphObj->CreateReferencedPropertiesNode(ReferencingPropertiesArray, ReferencingNode, ReferencedNode);
				}
			}
		}
	}
}

bool SReferenceViewer::CanResolveReferencingProperties() const
{
	if (!GraphEditorPtr)
	{
		return false;
	}

	return GraphEditorPtr->GetSelectedNodes().Num() >= 1;
}

void SReferenceViewer::HandleOnSearchTextChanged(const FText& SearchText)
{
	if (GraphObj == nullptr || !GraphEditorPtr.IsValid())
	{
		return;
	}

	GraphEditorPtr->ClearSelectionSet();

	UpdateIsPassingSearchFilterCallback();

	if (SearchText.IsEmpty())
	{
		constexpr bool bOnlySelection = false;
		// Zoom back to show the entire graph if nothing is selected
		GraphEditorPtr->ZoomToFit(bOnlySelection);
		return;
	}

	FString SearchString = SearchText.ToString();
	TArray<FString> SearchWords;
	SearchString.ParseIntoArrayWS( SearchWords );

	TArray<UEdGraphNode_Reference*> AllNodes;
	GraphObj->GetNodesOfClass<UEdGraphNode_Reference>( AllNodes );

	for (UEdGraphNode_Reference* Node : AllNodes)
	{
		if (DoesAssetPassSearchTextFilter(Node->GetIdentifier(), Node->GetAssetData(), bShowingContentVersePath, SearchWords))
		{
			constexpr bool bSelect = true;
			GraphEditorPtr->SetNodeSelection(Node, bSelect);
		}
	}

	constexpr bool bOnlySelection = true;
	// Zoom to fit the select nodes. Also ensures the graph is up to date
	GraphEditorPtr->ZoomToFit(bOnlySelection);
}

void SReferenceViewer::HandleOnSearchTextCommitted(const FText& SearchText, ETextCommit::Type CommitType)
{
	if (!GraphEditorPtr.IsValid())
	{
		return;
	}

	if (CommitType == ETextCommit::OnCleared)
	{
		GraphEditorPtr->ClearSelectionSet();

		constexpr bool bOnlySelection = true;
		GraphEditorPtr->ZoomToFit(bOnlySelection);
	}
	else if (CommitType == ETextCommit::OnEnter)
	{
		HandleOnSearchTextChanged(SearchBox->GetText());
	}

	FSlateApplication::Get().SetKeyboardFocus(GraphEditorPtr, EFocusCause::SetDirectly);
}

TSharedRef<SWidget> SReferenceViewer::GetShowMenuContent()
{
 	FMenuBuilder MenuBuilder(true, ReferenceViewerActions);

	MenuBuilder.BeginSection("ReferenceTypes", LOCTEXT("ReferenceTypes", "Reference Types"));
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().ShowSoftReferences);
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().ShowHardReferences);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("EditorOnlyReferenceTypes", LOCTEXT("EditorOnlyReferenceTypes", "Editor Only Reference Types"));
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().EditorOnlyReferenceFilterTypeGame);
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().EditorOnlyReferenceFilterTypePropagation);
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().EditorOnlyReferenceFilterTypeEditorOnly);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("Assets", LOCTEXT("Assets", "Assets"));
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().ShowManagementReferences);
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().ShowNameReferences);
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().ShowCodePackages);
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection("ViewOptions", LOCTEXT("ViewOptions", "View Options"));
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().ShowDuplicates);
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().FilterSearch);
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().CompactMode);
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().ShowExternalReferencers);
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().ShowCommentPath);
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SReferenceViewer::MakeToolBar()
{

	FToolBarBuilder ToolBarBuilder(ReferenceViewerActions, FMultiBoxCustomization::None, TSharedPtr<FExtender>(), true);
	ToolBarBuilder.SetStyle(&FReferenceViewerStyle::Get(), "AssetEditorToolbar");
	ToolBarBuilder.BeginSection("Test");

	ToolBarBuilder.AddToolBarButton(
		FUIAction(FExecuteAction::CreateSP(this, &SReferenceViewer::RefreshClicked)),
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FReferenceViewerStyle::Get().GetStyleSetName(), "Icons.Refresh"));

	ToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SReferenceViewer::BackClicked),
			FCanExecuteAction::CreateSP(this, &SReferenceViewer::IsBackEnabled)
		),
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>::CreateSP(this, &SReferenceViewer::GetHistoryBackTooltip),
		FSlateIcon(FReferenceViewerStyle::Get().GetStyleSetName(), "Icons.ArrowLeft"));

	ToolBarBuilder.AddToolBarButton(
		FUIAction(
			FExecuteAction::CreateSP(this, &SReferenceViewer::ForwardClicked),
			FCanExecuteAction::CreateSP(this, &SReferenceViewer::IsForwardEnabled)
		),
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>::CreateSP(this, &SReferenceViewer::GetHistoryForwardTooltip),
		FSlateIcon(FReferenceViewerStyle::Get().GetStyleSetName(), "Icons.ArrowRight"));

	ToolBarBuilder.AddToolBarButton(FAssetManagerEditorCommands::Get().FindPath,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>(), 
		FSlateIcon(FReferenceViewerStyle::Get().GetStyleSetName(), "BlueprintEditor.FindInBlueprint"));

	ToolBarBuilder.AddSeparator();

	ToolBarBuilder.AddComboButton( 
		FUIAction(),
		FOnGetContent::CreateSP(this, &SReferenceViewer::GetShowMenuContent),
		TAttribute<FText>(),
		TAttribute<FText>(),
		FSlateIcon(FReferenceViewerStyle::Get().GetStyleSetName(), "Icons.Visibility"),
		/*bInSimpleComboBox*/ false);

	ToolBarBuilder.AddToolBarButton(FAssetManagerEditorCommands::Get().ShowDuplicates,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>::CreateLambda([this]() -> FText
		{ 
			if (Settings->GetFindPathEnabled())
			{
				return LOCTEXT("DuplicatesDisabledTooltip", "Duplicates are always shown when using the Find Path tool.");
			}

			return FAssetManagerEditorCommands::Get().ShowDuplicates->GetDescription();
		}),

		FSlateIcon(FReferenceViewerStyle::Get().GetStyleSetName(), "Icons.Duplicate"));

	ToolBarBuilder.AddSeparator();

	ToolBarBuilder.AddToolBarButton(FAssetManagerEditorCommands::Get().Filters,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>::CreateLambda([this]() -> FText
		{ 
			if (Settings->GetFindPathEnabled())
			{
				return LOCTEXT("FiltersDisabledTooltip", "Filtering is disabled when using the Find Path tool.");
			}

			return FAssetManagerEditorCommands::Get().Filters->GetDescription();
		}),

		FSlateIcon(FReferenceViewerStyle::Get().GetStyleSetName(), "Icons.Filters"));

	ToolBarBuilder.AddToolBarButton(FAssetManagerEditorCommands::Get().AutoFilters,
		NAME_None,
		TAttribute<FText>(),
		TAttribute<FText>::CreateLambda([this]() -> FText
		{ 
			if (Settings->GetFindPathEnabled())
			{
				return LOCTEXT("AutoFiltersDisabledTooltip", "AutoFiltering is disabled when using the Find Path tool.");
			}

			return FAssetManagerEditorCommands::Get().AutoFilters->GetDescription();
		}),

		FSlateIcon(FReferenceViewerStyle::Get().GetStyleSetName(), "Icons.AutoFilters"));

	ToolBarBuilder.EndSection();


	return ToolBarBuilder.MakeWidget();
}

TSharedRef<SWidget> SReferenceViewer::GenerateFindPathAssetPickerMenu()
{
	FAssetPickerConfig AssetPickerConfig;
	AssetPickerConfig.Filter.bRecursiveClasses = true;
	AssetPickerConfig.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SReferenceViewer::OnFindPathAssetSelected);
	AssetPickerConfig.OnAssetEnterPressed = FOnAssetEnterPressed::CreateSP(this, &SReferenceViewer::OnFindPathAssetEnterPressed);
	AssetPickerConfig.InitialAssetViewType = EAssetViewType::List;
	AssetPickerConfig.bAllowNullSelection = true;
	AssetPickerConfig.bFocusSearchBoxWhenOpened = true;
	AssetPickerConfig.bAllowDragging = false;

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));


	return SNew(SBox)
	.HeightOverride(500)
	[
		SNew( SBorder )
		.BorderImage( FAppStyle::GetBrush("Menu.Background") )
		[
			ContentBrowserModule.Get().CreateAssetPicker(AssetPickerConfig)
		]
	];
}

void SReferenceViewer::OnFindPathAssetSelected( const FAssetData& AssetData )
{
	FindPathAssetPicker->SetIsOpen(false);

	FindPathAssetId = FAssetIdentifier(AssetData.PackageName);

	const TArray<FAssetIdentifier>& CurrentGraphRootIdentifiers = GraphObj->GetCurrentGraphRootIdentifiers();
	if (!CurrentGraphRootIdentifiers.IsEmpty())
	{
		GraphObj->FindPath(CurrentGraphRootIdentifiers[0], FindPathAssetId);
	}

	RegisterActiveTimer(0.1f, FWidgetActiveTimerDelegate::CreateSP(this, &SReferenceViewer::TriggerZoomToFit));
}

void SReferenceViewer::OnFindPathAssetEnterPressed( const TArray<FAssetData>& AssetData )
{
	FindPathAssetPicker->SetIsOpen(false);

	if (!AssetData.IsEmpty())
	{
		FindPathAssetId = FAssetIdentifier(AssetData[0].PackageName);

		const TArray<FAssetIdentifier>& CurrentGraphRootIdentifiers = GraphObj->GetCurrentGraphRootIdentifiers();
		if (!CurrentGraphRootIdentifiers.IsEmpty())
		{
			GraphObj->FindPath(CurrentGraphRootIdentifiers[0], FindPathAssetId);
		}
	}

	RegisterActiveTimer(0.1f, FWidgetActiveTimerDelegate::CreateSP(this, &SReferenceViewer::TriggerZoomToFit));
} 

#undef LOCTEXT_NAMESPACE
