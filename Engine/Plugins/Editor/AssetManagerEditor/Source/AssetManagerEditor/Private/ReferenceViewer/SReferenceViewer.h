// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GraphEditor.h"
#include "AssetRegistry/AssetData.h"
#include "HistoryManager.h"
#include "CollectionManagerTypes.h"
#include "ReferenceViewer/ReferenceViewerSettings.h"

class ICollectionContainer;
class IPlugin;
class FUICommandList;
class SComboButton;
class SDockTab;
class SReferenceViewerFilterBar;
class SSearchBox;
class UEdGraph_ReferenceViewer;
class UReferenceViewerSettings;
namespace ESelectInfo { enum Type : int; }
struct FAssetManagerEditorRegistrySource;
template <typename OptionType> class SComboBox;

class UEdGraph;

DECLARE_MULTICAST_DELEGATE_TwoParams(FOnReferenceViewerSelectionChanged, const TArray<FAssetIdentifier>&, const TArray<FAssetIdentifier>&)

/**
 * 
 */
class SReferenceViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS( SReferenceViewer ){}
		SLATE_ARGUMENT(TSharedPtr<SDockTab>, ParentTab)
	SLATE_END_ARGS()

	~SReferenceViewer();

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	/**
	 * Sets a new root package name
	 *
	 * @param NewGraphRootIdentifiers	The root elements of the new graph to be generated
	 * @param ReferenceViewerParams		Different visualization settings, such as whether it should display the referencers or the dependencies of the NewGraphRootIdentifiers
	 */
	void SetGraphRootIdentifiers(const TArray<FAssetIdentifier>& NewGraphRootIdentifiers, const FReferenceViewerParams& ReferenceViewerParams = FReferenceViewerParams());

	/** Gets graph editor */
	TSharedPtr<SGraphEditor> GetGraphEditor() const { return GraphEditorPtr; }

	/** Called when the current registry source changes */
	void SetCurrentRegistrySource(const FAssetManagerEditorRegistrySource* RegistrySource);

	/**SWidget interface **/
	virtual bool SupportsKeyboardFocus() const override { return true; }
	virtual FReply OnKeyDown( const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent ) override;
	virtual void Tick( const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime ) override;

	bool CanMakeCollectionWithReferencersOrDependencies(TSharedPtr<ICollectionContainer> CollectionContainer, ECollectionShareType::Type ShareType) const;
	void MakeCollectionWithReferencersOrDependencies(TSharedPtr<ICollectionContainer> CollectionContainer, ECollectionShareType::Type ShareType, bool bReferencers);

	FOnReferenceViewerSelectionChanged& OnReferenceViewerSelectionChanged() { return OnReferenceViewerSelectionChangedDelegate; }

	void GetSelectedNodeAssetData(TArray<FAssetData>& OutAssetData) const;
private:

	/** Call after a structural change is made that causes the graph to be recreated */
	void RebuildGraph();

	/** Called to create context menu when right-clicking on graph */
	FActionMenuContent OnCreateGraphActionMenu(UEdGraph* InGraph, const FVector2f& InNodePosition, const TArray<UEdGraphPin*>& InDraggedPins, bool bAutoExpand, SGraphEditor::FActionMenuClosed InOnMenuClosed);

	/** Called when a node is double clicked */
	void OnNodeDoubleClicked(class UEdGraphNode* Node);

	/** True if the user may use the history back button */
	bool IsBackEnabled() const;

	/** True if the user may use the history forward button */
	bool IsForwardEnabled() const;

	/** Handler for clicking the history back button */
	void BackClicked();

	/** Handler for clicking the history forward button */
	void ForwardClicked();

	/** Refresh the current view */
	void RefreshClicked();

	/** Handler for when the graph panel tells us to go back in history (like using the mouse thumb button) */
	void GraphNavigateHistoryBack();

	/** Handler for when the graph panel tells us to go forward in history (like using the mouse thumb button) */
	void GraphNavigateHistoryForward();

	/** Gets the tool tip text for the history back button */
	FText GetHistoryBackTooltip() const;

	/** Gets the tool tip text for the history forward button */
	FText GetHistoryForwardTooltip() const;

	/** Gets the text to be displayed in the address bar */
	FText GetAddressBarText() const;

	/** Gets the summary text to be displayed for multiple FAssetIdentifiers */
	FText GetIdentifierSummaryText(const TArray<FAssetIdentifier>& Identifiers) const;

	/** Gets the text to be displayed for a FAssetIdentifier */
	FText GetIdentifierText(const FAssetIdentifier& Identifier) const;

	/** Gets the text to be displayed for warning/status updates */
	FText GetStatusText() const;
	
	/** Gets the text to be displayed at the center of the graph */
    FText GetCenteredStatusText() const;

	/** Gets the visibility for the text which can be displayed at the center of the graph */
	EVisibility GetCenteredStatusVisibility() const;

	/** Called when the path is being edited */
	void OnAddressBarTextChanged(const FText& NewText);

	/** Sets the new path for the viewer */
	void OnAddressBarTextCommitted(const FText& NewText, ETextCommit::Type CommitInfo);

	void OnApplyHistoryData(const FReferenceViewerHistoryData& History);

	void OnUpdateHistoryData(FReferenceViewerHistoryData& HistoryData) const;

	void OnUpdateFilterBar();
	
	void OnSearchDepthEnabledChanged( ECheckBoxState NewState );
	ECheckBoxState IsSearchDepthEnabledChecked() const;

	int32 GetSearchReferencerDepthCount() const;
	int32 GetSearchDependencyDepthCount() const;

	void OnSearchReferencerDepthCommitted(int32 NewValue);
	void OnSearchDependencyDepthCommitted(int32 NewValue);

	void OnEnableCollectionFilterChanged(ECheckBoxState NewState);
	ECheckBoxState IsEnableCollectionFilterChecked() const;
	void CollectionFilterAddMenuEntry(FMenuBuilder& MenuBuilder, const TSharedPtr<ICollectionContainer>& CollectionContainer,  const FName& CollectionName);
	TSharedRef<SWidget> BuildCollectionFilterMenu();
	FText GetCollectionComboButtonText() const;

	void OnEnablePluginFilterChanged(ECheckBoxState NewState);
	ECheckBoxState IsEnablePluginFilterChecked() const;
	void PluginFilterAddMenuEntry(FMenuBuilder& MenuBuilder, const FName& PluginName, const FText& Label, const FText& ToolTip);
	TSharedRef<SWidget> BuildPluginFilterMenu();
	FText GetPluginComboButtonText() const;

	void OnShowSoftReferencesChanged();
	bool IsShowSoftReferencesChecked() const;
	void OnShowHardReferencesChanged();
	bool IsShowHardReferencesChecked() const;
	void OnEditorOnlyReferenceFilterTypeChanged(EEditorOnlyReferenceFilterType Value);
	EEditorOnlyReferenceFilterType GetEditorOnlyReferenceFilterType() const;

	void OnShowFilteredPackagesOnlyChanged();
	bool IsShowFilteredPackagesOnlyChecked() const;
	void UpdateIsPassingSearchFilterCallback();

	void OnCompactModeChanged();
	bool IsCompactModeChecked() const;

	void OnShowExternalReferencersChanged();
	bool IsShowExternalReferencersChecked() const;

	void OnShowDuplicatesChanged();
	bool IsShowDuplicatesChecked() const;

	bool GetManagementReferencesVisibility() const;
	void OnShowManagementReferencesChanged();
	bool IsShowManagementReferencesChecked() const;
	void OnShowSearchableNamesChanged();
	bool IsShowSearchableNamesChecked() const;
	void OnShowCodePackagesChanged();
	bool IsShowCodePackagesChecked() const;

	int32 GetSearchBreadthCount() const;
	void SetSearchBreadthCount(int32 InBreadthValue);
	void OnSearchBreadthChanged(int32 InBreadthValue);
	void OnSearchBreadthCommited(int32 InBreadthValue, ETextCommit::Type InCommitType);

	TSharedRef<SWidget> GetShowMenuContent();

	void RegisterActions();
	void ShowSelectionInContentBrowser();
	void OpenSelectedInAssetEditor();
	void ReCenterGraph();
	void CopyReferencedObjects();
	void CopyReferencingObjects();
	void ShowReferencedObjects();
	void ShowReferencingObjects();
	bool CanMakeCollectionWithReferencersOrDependencies(ECollectionShareType::Type ShareType) const;
	void MakeCollectionWithReferencersOrDependencies(ECollectionShareType::Type ShareType, bool bReferencers);
	void ShowReferenceTree();
	void ViewSizeMap();
	void ViewAssetAudit();
	void ZoomToFit();
	bool CanZoomToFit() const;
	void OnFind();
	void ResolveReferencingProperties() const;
	bool CanResolveReferencingProperties() const;

	/** Find Path */
	TSharedRef<SWidget> GenerateFindPathAssetPickerMenu();
	void OnFindPathAssetSelected( const FAssetData& AssetData );
	void OnFindPathAssetEnterPressed( const TArray<FAssetData>& AssetData );
	TSharedPtr<SComboButton> FindPathAssetPicker;
	FAssetIdentifier FindPathAssetId;

	/** Handlers for searching */
	void HandleOnSearchTextChanged(const FText& SearchText);
	void HandleOnSearchTextCommitted(const FText& SearchText, ETextCommit::Type CommitType);

	void ReCenterGraphOnNodes(const TSet<UObject*>& Nodes);

	enum class EObjectsListType
	{
		Referenced,
		Referencing,
	};

	FString GetObjectsList(EObjectsListType ObjectsListType) const;

	UObject* GetObjectFromSingleSelectedNode() const;
	void GetPackageNamesFromSelectedNodes(TSet<FName>& OutNames) const;
	bool HasExactlyOneNodeSelected() const;
	bool HasExactlyOnePackageNodeSelected() const;
	bool HasAtLeastOnePackageNodeSelected() const;
	bool HasAtLeastOneRealNodeSelected() const;

	void OnAssetRegistryChanged(const FAssetData& AssetData);
	void OnInitialAssetRegistrySearchComplete();
	void OnPluginEdited(IPlugin& InPlugin);

	EActiveTimerReturnType TriggerZoomToFit(double InCurrentTime, float InDeltaTime);
private:

	TSharedRef<SWidget> MakeToolBar();


	/** The manager that keeps track of history data for this browser */
	FReferenceViewerHistoryManager HistoryManager;

	TSharedPtr<SGraphEditor> GraphEditorPtr;

	TSharedPtr<FUICommandList> ReferenceViewerActions;
	TSharedPtr<SSearchBox> SearchBox;
	TSharedPtr<SWidget> ReferencerCountBox;
	TSharedPtr<SWidget> DependencyCountBox;
	TSharedPtr<SWidget> BreadthLimitBox;

	TSharedPtr< SReferenceViewerFilterBar > FilterWidget;

	UEdGraph_ReferenceViewer* GraphObj;

	UReferenceViewerSettings* Settings;

	/** The temporary copy of the path text when it is actively being edited. */
	FText TemporaryPathBeingEdited;

	/** Combo box for collections filter options */
	TSharedPtr<SComboBox<TSharedPtr<FName>>> CollectionsCombo;

	/** List of collection filter options */
	TArray<TSharedPtr<FName>> CollectionsComboList;

	/**
	 * Whether to visually show to the user the option of "Search Depth Limit" or hide it and fix it to a default value:
	 * - If 0 or negative, it will show to the user the option of "Search Depth Limit".
	 * - If >0, it will hide that option and fix the Depth value to this value.
	 */
	int32 FixAndHideSearchDepthLimit;
	/**
	 * Whether to visually show to the user the option of "Search Breadth Limit" or hide it and fix it to a default value:
	 * - If 0 or negative, it will show to the user the option of "Search Breadth Limit".
	 * - If >0, it will hide that option and fix the Breadth value to this value.
	 */
	int32 FixAndHideSearchBreadthLimit;
	/** Whether to visually show to the user the option of "Collection Filter" */
	bool bShowCollectionFilter;
	/** Whether to visually show to the user the option of "Plugin Filter" */
	bool bShowPluginFilter;
	/** Whether to visually show to the user the options of "Show Soft/Hard/Management References" */
	bool bShowShowReferencesOptions;
	/** Whether to visually show to the user the option of "Show Searchable Names" */
	bool bShowShowSearchableNames;
	/** Whether to visually show to the user the option of "Show C++ Packages" */
	bool bShowShowCodePackages;
	/** Whether to visually show to the user the option of "Show Filtered Packages Only" */
	bool bShowShowFilteredPackagesOnly;
	/** True if our view is out of date due to asset registry changes */
	bool bDirtyResults;
	/** Whether to visually show to the user the option of "Compact Mode" */
	bool bShowCompactMode;

	/** Whether to show Verse paths */
	bool bShowingContentVersePath;

	/** A recursion check so as to avoid the rebuild of the graph if we are currently rebuilding the filters */
	bool bRebuildingFilters;

	/** Used to delay graph rebuilding during spinbox slider interaction */
	bool bNeedsGraphRebuild;
	bool bNeedsGraphRefilter;
	bool bNeedsReferencedPropertiesUpdate;
	double SliderDelayLastMovedTime = 0.0;
	double GraphRebuildSliderDelay = 0.25;

	/** Handle to know if dirty */
	FDelegateHandle AssetRefreshHandle;

	/** Called when expanding a node, or manually updating the asset path */
	FOnReferenceViewerSelectionChanged OnReferenceViewerSelectionChangedDelegate;
};

enum class EDependencyPinCategory
{
	LinkEndPassive = 0,
	LinkEndActive = 1,
	LinkEndMask = LinkEndActive,

	LinkTypeNone = 0,
	LinkTypeUsedInGame = 2,
	LinkTypeHard = 4,
	LinkTypeMask = LinkTypeHard | LinkTypeUsedInGame,
};
ENUM_CLASS_FLAGS(EDependencyPinCategory);

extern EDependencyPinCategory ParseDependencyPinCategory(FName PinCategory);
extern FLinearColor GetColor(EDependencyPinCategory Category);
extern FName GetName(EDependencyPinCategory Category);
