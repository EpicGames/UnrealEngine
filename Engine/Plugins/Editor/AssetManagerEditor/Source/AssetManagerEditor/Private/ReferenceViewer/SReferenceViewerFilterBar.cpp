// Copyright Epic Games, Inc. All Rights Reserved.

#include "ReferenceViewer/SReferenceViewerFilterBar.h"

#include "ReferenceViewer/ReferenceViewerSettings.h"
#include "ReferenceViewerFilters.h"

#define LOCTEXT_NAMESPACE "ReferenceViewerFilterBar"

void SReferenceViewerFilterBar::Construct(const FArguments& InArgs)
{
	SFilterBar::Construct(InArgs);

	TSharedPtr<FFilterCategory> DefaultCategory = MakeShareable( new FFilterCategory(LOCTEXT("ReferenceViewerFiltersCategory", "Other Filters"), LOCTEXT("ReferenceViewerFiltersCategoryTooltip", "Filter nodes by all filters in this category.")) );

	AddFilter(MakeShareable(new FReferenceViewerFilter_ShowCheckedOut(DefaultCategory)));
}

void SReferenceViewerFilterBar::SaveSettings()
{
	if (UReferenceViewerSettings* Settings = GetMutableDefault<UReferenceViewerSettings>())
	{
		// Only save & load user filters, not autofilters
		if (!Settings->AutoUpdateFilters())
		{
			TArray<FilterState> SaveFilters;
			for (const TSharedRef<SAssetFilter>& CurrentAssetFilter : AssetFilters)
			{
				FTopLevelAssetPath AssetFilterPath = CurrentAssetFilter->GetCustomClassFilterData()->GetClassPathName();
				SaveFilters.Add(FilterState(AssetFilterPath, CurrentAssetFilter->IsEnabled()));
			}
			Settings->SetUserFilters(SaveFilters);

			TMap<FString, bool> CustomFilters;
			for (const TSharedRef<SFilter>& ActiveFilter : Filters)
			{
				CustomFilters.Add(ActiveFilter->GetFilterName(), ActiveFilter->IsEnabled());
			}
			Settings->SetCustomFilters(CustomFilters);
		}
	}
}

void SReferenceViewerFilterBar::LoadSettings() 
{
	if (UReferenceViewerSettings* Settings = GetMutableDefault<UReferenceViewerSettings>())
	{
		// Only save & load user filters, not autofilters
		if (!Settings->AutoUpdateFilters())
		{
			TArray<FilterState> SavedFilters = Settings->GetUserFilters();
			RemoveAllFilters();
			for (FilterState& State : SavedFilters)
			{
				if (DoesAssetTypeFilterExist(State.FilterPath))
				{
					SetAssetTypeFilterCheckState(State.FilterPath, ECheckBoxState::Checked);
					ToggleAssetTypeFilterEnabled(State.FilterPath, State.bIsEnabled);
				}
			}

			const TMap<FString, bool>& CustomFilters = Settings->GetCustomFilters();
			for (const TPair<FString, bool>& FilterElem : CustomFilters)
			{
				TSharedRef<FFilterBase<FReferenceNodeInfo&>>* FilterRefPtr = AllFrontendFilters.FindByPredicate(
					[FilterName = FilterElem.Key](const TSharedRef<FFilterBase<FReferenceNodeInfo&>>& InFilter)
					{
						return InFilter->GetName().Equals(FilterName);
					}
				);

				if (FilterRefPtr)
				{
					TSharedRef<FFilterBase<FReferenceNodeInfo&>> FilterRef = *FilterRefPtr;
					AddFilterToBar(FilterRef);
					FilterRef->SetActive(FilterElem.Value);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
