// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Filters/SFilterBar.h"

struct FReferenceNodeInfo;

class SReferenceViewerFilterBar : public SFilterBar< FReferenceNodeInfo& > 
{

public:

	/** Constructs this widget with InArgs */
	void Construct( const FArguments& InArgs );

	/** Saves any settings to config that should be persistent between editor sessions */
	void SaveSettings() override;

	/** Loads any settings to config that should be persistent between editor sessions */
	void LoadSettings() override;

};
