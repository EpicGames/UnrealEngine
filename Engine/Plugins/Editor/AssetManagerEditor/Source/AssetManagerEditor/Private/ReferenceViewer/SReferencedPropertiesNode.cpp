// Copyright Epic Games, Inc. All Rights Reserved.

#include "SReferencedPropertiesNode.h"
#include "ReferenceViewer/EdGraphNode_Reference.h"
#include "ReferenceViewer/EdGraphNode_ReferencedProperties.h"
#include "ReferenceViewerStyle.h"
#include "SlateOptMacros.h"
#include "Styling/SlateIconFinder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "ReferencedPropertyNode"

namespace UE::ReferenceViewer::Private
{
FText GetInvalidReferenceDescription()
{
	return LOCTEXT("ReferenceNameInvalidTooltip", "Invalid reference description");
}
}

/**
 * Widget representing a referencing property
 */
class SReferencedPropertyNode : public STableRow<FReferencingPropertyDescriptionPtr>
{
public:
	SLATE_BEGIN_ARGS(SReferencedPropertyNode)
		{
		}

	SLATE_END_ARGS()
	void Construct(const FArguments& InArgs, const FReferencingPropertyDescriptionPtr& InReferencingPropertyDescription, const TSharedRef<STableViewBase>& InOwnerTableView);

private:
	FText GetPropertyDisplayName() const;
	FText GetTooltipText() const;
	FText GetIndirectReferenceTooltipText() const;
	const FSlateBrush* GetIconBrush() const;
	const FSlateBrush* GetIndirectReferenceIconBrush() const;
	EVisibility GetIndirectReferenceVisibility() const;

	TWeakPtr<FReferencingPropertyDescription> PropertyDescription;
};

void SReferencedPropertiesNode::Construct(const FArguments& InArgs, UEdGraphNode_ReferencedProperties* InReferencedPropertiesNode)
{
	GraphNode = InReferencedPropertiesNode;

	if (InReferencedPropertiesNode)
	{
		InReferencedPropertiesNode->OnPropertiesDescriptionUpdated().AddRaw(this, &SReferencedPropertiesNode::UpdateGraphNode);
	}

	UpdateGraphNode();
}

SReferencedPropertiesNode::~SReferencedPropertiesNode()
{
	if (UEdGraphNode_ReferencedProperties* ReferencedProperties = Cast<UEdGraphNode_ReferencedProperties>(GraphNode))
	{
		ReferencedProperties->OnPropertiesDescriptionUpdated().RemoveAll(this);
	}
}

void SReferencedPropertiesNode::UpdateGraphNode()
{
	// No pins
	InputPins.Empty();
	OutputPins.Empty();

	// No side Boxes
	RightNodeBox.Reset();
	LeftNodeBox.Reset();

	TSharedPtr<SWidget> MainWidget;

	if (const UEdGraphNode_ReferencedProperties* ReferencedProperties = Cast<UEdGraphNode_ReferencedProperties>(GraphNode))
	{
		ReferencingPropertiesSource = ReferencedProperties->GetReferencedPropertiesDescription();

		PropertiesTreeView = SNew(SListView<FReferencingPropertyDescriptionPtr>)
				.ConsumeMouseWheel(EConsumeMouseWheel::Always)
				.OnGenerateRow(this, &SReferencedPropertiesNode::OnGenerateRow)
				.SelectionMode(ESelectionMode::None)
				.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("SimpleListView"))
				.Orientation(Orient_Vertical)
				.ListItemsSource(&ReferencingPropertiesSource);

		if (!ReferencingPropertiesSource.IsEmpty())
		{
			MainWidget = PropertiesTreeView;
			PropertiesTreeView->RequestListRefresh();
		}
		else
		{
			MainWidget = 
            	SNew(STextBlock)
            	.TextStyle(FReferenceViewerStyle::Get(), "Graph.ReferencedPropertiesText")
	            .Text(LOCTEXT("ReferencingPropertyDataUnavailable", "Impossible to retrieve at this time."));
		}
	}
	else
	{
		MainWidget = SNullWidget::NullWidget;
	}

	const FButtonStyle* CloseButtonStyle =
		&FReferenceViewerStyle::Get().GetWidgetStyle<FButtonStyle>(TEXT("Graph.ReferencedPropertiesCloseButton"));

	// clang-format off
	GetOrAddSlot(ENodeZone::Center)
	.HAlign(HAlign_Center)
	.VAlign(VAlign_Center)
	[
		SNew(SBorder)
		.ToolTipText(this, &SReferencedPropertiesNode::GetTooltipText)
		.ColorAndOpacity(FLinearColor::White)
		.BorderImage(FReferenceViewerStyle::Get().GetBrush("Graph.Node.BodyBackground"))
		.Padding(0.0f)
		[
			SNew(SBorder)
			.BorderImage(FReferenceViewerStyle::Get().GetBrush("Graph.Node.BodyBorder"))
			.BorderBackgroundColor(FLinearColor(0.5f, 0.5f, 0.5f, 0.4f))
			.Padding(0.0f)
			[
				SNew(SBorder)
				.BorderImage(FReferenceViewerStyle::Get().GetBrush("Graph.Node.Body"))
				.Padding(0.0f)
				[
					SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					.AutoHeight()
					[
						SNew(SBox)
						.Padding(4.0f)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Left)
							.AutoWidth()
							.Padding(4.0, 2.0f, 0.0f, 0.0f)
							[
								SNew(STextBlock)
								.TextStyle(&FReferenceViewerStyle::Get(), TEXT("Graph.Node.NodeTitleExtraLines"))
								.Text(LOCTEXT("ReferencingPropertiesLabel", "Referencing Properties"))
							]
							+ SHorizontalBox::Slot()
							.HAlign(HAlign_Right)
							[
								SNew(SButton)
								.ContentPadding(1.0f)
								.ButtonStyle(CloseButtonStyle)
								.OnClicked(this, &SReferencedPropertiesNode::CloseNode)
								[
									SNew(SSpacer)
									.Size(CloseButtonStyle->Normal.ImageSize)
								]
							]
						]
					]
					+ SVerticalBox::Slot()
					[
						SNew(SBox)
						.MaxDesiredHeight(200.0f)
						.MinDesiredWidth(200.0f)
						.Padding(2.0f)
						[
							MainWidget.ToSharedRef()
						]
					]
				]
			]
		]
	];
	// clang-format on
}

void SReferencedPropertiesNode::Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SGraphNode::Tick(InAllottedGeometry, InCurrentTime, InDeltaTime);

	if (UEdGraphNode_ReferencedProperties* ReferencedProperties = Cast<UEdGraphNode_ReferencedProperties>(GraphNode))
	{
		const FVector2f& Size = InAllottedGeometry.GetLocalSize();
		ReferencedProperties->RefreshLocation(Size);
	}
}

TSharedRef<ITableRow> SReferencedPropertiesNode::OnGenerateRow(
	FReferencingPropertyDescriptionPtr ReferencingPropertyDescription, const TSharedRef<STableViewBase>& TableViewBase
) const
{
	return SNew(SReferencedPropertyNode, ReferencingPropertyDescription, TableViewBase);
}

FReply SReferencedPropertiesNode::CloseNode() const
{
	if (UEdGraphNode_ReferencedProperties* ReferencedPropertiesNode = Cast<UEdGraphNode_ReferencedProperties>(GraphNode))
	{
		if (UEdGraph_ReferenceViewer* Graph = Cast<UEdGraph_ReferenceViewer>(GraphNode->GetGraph()))
		{
			Graph->CloseReferencedPropertiesNode(ReferencedPropertiesNode);
		}
	}

	return FReply::Handled();
}

FText SReferencedPropertiesNode::GetTooltipText() const
{
	if (UEdGraphNode_ReferencedProperties* ReferencedPropertiesNode = Cast<UEdGraphNode_ReferencedProperties>(GraphNode))
	{
		TObjectPtr<UEdGraphNode_Reference> ReferencedNode = ReferencedPropertiesNode->GetReferencedNode();
		TObjectPtr<UEdGraphNode_Reference> ReferencingNode = ReferencedPropertiesNode->GetReferencingNode();

		if (ReferencedNode && ReferencingNode)
		{
			FFormatNamedArguments Arguments;
			Arguments.Add(TEXT("0"), FText::FromName(ReferencingNode->GetAssetData().AssetName));
			Arguments.Add(TEXT("1"), FText::FromName(ReferencedNode->GetAssetData().AssetName));

			return FText::Format(LOCTEXT("ReferencingPropertiesNodeTooltip", "{0} properties referencing {1}"), Arguments);
		}
	}

	return FText();
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SReferencedPropertyNode::Construct(const FArguments& InArgs, const FReferencingPropertyDescriptionPtr& InReferencingPropertyDescription, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	STableRow::FArguments Args = STableRow::FArguments();
	Args.Style(&FReferenceViewerStyle::Get().GetWidgetStyle<FTableRowStyle>(TEXT("Graph.ReferencedPropertiesTableRow")));

	STableRow<FReferencingPropertyDescriptionPtr>::Construct(Args, InOwnerTableView);
	PropertyDescription = InReferencingPropertyDescription;

	// clang-format off
	ChildSlot
	.Padding(FMargin(6.0f, 4.0f, 6.0f, 4.0f))
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.VAlign(VAlign_Center)
		.Padding(FMargin(0.f, 0.f, 8.f, 0.f))
		.AutoWidth()
		[
			SNew(SImage)
			.Image(GetIconBrush())
			.DesiredSizeOverride(FVector2D(16.0f, 16.0f))
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1.0)
		.VAlign(VAlign_Center)
		[
			SNew(SBox)
			.ToolTipText_Raw(this, &SReferencedPropertyNode::GetTooltipText)
			[
				SNew(STextBlock)
				.TextStyle(FReferenceViewerStyle::Get(), "Graph.ReferencedPropertiesText")
				.Text(this, &SReferencedPropertyNode::GetPropertyDisplayName)
			]
		]
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		.Padding(FMargin(6.f, 0.f, 0.f, 0.f))
		[
			SNew(SBox)
			.ToolTipText(this, &SReferencedPropertyNode::GetIndirectReferenceTooltipText)
			.Visibility(this, &SReferencedPropertyNode::GetIndirectReferenceVisibility)
			[
				SNew(SImage)
				.Image(GetIndirectReferenceIconBrush())
				.DesiredSizeOverride(FVector2D(10.0f, 10.0f))
			]
		]
	];
	// clang-format on
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FText SReferencedPropertyNode::GetPropertyDisplayName() const
{
	if (TSharedPtr<FReferencingPropertyDescription> PropertyDescriptionPinned = PropertyDescription.Pin())
	{
		return FText::FromString(PropertyDescriptionPinned->GetName());
	}

	return FText();
}

FText SReferencedPropertyNode::GetTooltipText() const
{
	if (TSharedPtr<FReferencingPropertyDescription> PropertyDescriptionPinned = PropertyDescription.Pin())
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("0"), FText::FromString(PropertyDescriptionPinned->GetReferencedNodeName()));
		Arguments.Add(TEXT("1"), FText::FromString(PropertyDescriptionPinned->GetTypeAsString()));

		return FText::Format(LOCTEXT("ReferenceNameTooltip", "Reference to {0} used as {1}"), Arguments);
	}

	return UE::ReferenceViewer::Private::GetInvalidReferenceDescription();
}

FText SReferencedPropertyNode::GetIndirectReferenceTooltipText() const
{
	if (TSharedPtr<FReferencingPropertyDescription> PropertyDescriptionPinned = PropertyDescription.Pin())
	{
		FFormatNamedArguments Arguments;
		Arguments.Add(TEXT("0"), FText::FromString(PropertyDescriptionPinned->GetName()));
		Arguments.Add(TEXT("1"), FText::FromString(PropertyDescriptionPinned->GetReferencedNodeName()));

		return FText::Format(LOCTEXT("IndirectReferenceTooltip", "Indirect reference: {0} is referencing {1}"), Arguments);
	}

	return UE::ReferenceViewer::Private::GetInvalidReferenceDescription();
}

const FSlateBrush* SReferencedPropertyNode::GetIconBrush() const
{
	const FSlateBrush* ComponentIcon = nullptr;

	if (TSharedPtr<FReferencingPropertyDescription> PropertyDescriptionPinned = PropertyDescription.Pin())
	{
		const UClass* Class = PropertyDescriptionPinned->GetPropertyClass();

		if (Class)
		{
			ComponentIcon = FSlateIconFinder::FindIconBrushForClass(Class);
		}
		else
		{
			if (PropertyDescriptionPinned->GetType() == FReferencingPropertyDescription::EAssetReferenceType::Component)
			{
				ComponentIcon = FSlateIconFinder::FindIconBrushForClass(UActorComponent::StaticClass(), TEXT("SCS.Component"));
			}
		}
	}
	else
	{
		ComponentIcon = FSlateIconFinder::FindIconBrushForClass(UObject::StaticClass());
	}

	return ComponentIcon;
}

const FSlateBrush* SReferencedPropertyNode::GetIndirectReferenceIconBrush() const
{
	return FAppStyle::GetBrush("ReferenceViewer.IndirectReference");
}

EVisibility SReferencedPropertyNode::GetIndirectReferenceVisibility() const
{
	if (TSharedPtr<FReferencingPropertyDescription> PropertyDescriptionPinned = PropertyDescription.Pin())
	{
		return PropertyDescriptionPinned->IsIndirect() ? EVisibility::Visible : EVisibility::Collapsed;
	}
	return EVisibility::Collapsed;
}

#undef LOCTEXT_NAMESPACE
