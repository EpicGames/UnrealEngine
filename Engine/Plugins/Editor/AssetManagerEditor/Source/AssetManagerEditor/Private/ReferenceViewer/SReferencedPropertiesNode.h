// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ReferenceViewer/EdGraphNode_ReferencedProperties.h"
#include "ReferenceViewer/EdGraph_ReferenceViewer.h"
#include "SGraphNode.h"
#include "Widgets/Views/SListView.h"

class UEdGraphNode_ReferencedProperties;

/**
 * Widget displaying the list of properties referencing a specified Asset in the reference Viewer.
 * It visually represents an UEdGraphNode_ReferencedProperties node
 */
class SReferencedPropertiesNode : public SGraphNode
{
public:
	SLATE_BEGIN_ARGS(SReferencedPropertiesNode)
	{
	}

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, UEdGraphNode_ReferencedProperties* InReferencedPropertiesNode);

	~SReferencedPropertiesNode();

	// ~Begin SGraphNode
	virtual void UpdateGraphNode() override;
	virtual bool IsNodeEditable() const override { return false; }
	// ~End SGraphNode

	// ~Begin SNode
	virtual bool CanBeSelected(const FVector2f& InMousePositionInNode) const override { return false; }
	// ~End SNode

	// ~Begin SWidget
	virtual void Tick(const FGeometry& InAllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	// ~End SWidget

private:
	TSharedRef<ITableRow> OnGenerateRow(FReferencingPropertyDescriptionPtr ReferencingPropertyDescription, const TSharedRef<STableViewBase>& TableViewBase) const;
	FReply CloseNode() const;
	FText GetTooltipText() const;

	TArray<FReferencingPropertyDescriptionPtr> ReferencingPropertiesSource;
	TSharedPtr<SListView<FReferencingPropertyDescriptionPtr>> PropertiesTreeView;
};
