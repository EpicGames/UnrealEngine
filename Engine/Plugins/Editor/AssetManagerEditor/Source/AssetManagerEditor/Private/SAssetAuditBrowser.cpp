// Copyright Epic Games, Inc. All Rights Reserved.

#include "SAssetAuditBrowser.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetDefinitionRegistry.h"
#include "Blueprint/BlueprintSupport.h"
#include "ClassViewerModule.h"
#include "ContentBrowserDataSource.h"
#include "ContentBrowserModule.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "Editor.h"
#include "Engine/AssetManager.h"
#include "Engine/BlueprintCore.h"
#include "FileHelpers.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "IContentBrowserSingleton.h"
#include "Styling/StyleColors.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboBox.h"

#include "AssetManagerEditorModule.h"
#include "AssetManagerEditorCommands.h"
#include "InsightsCore/Common/Log.h"
#include "TreeView/AssetTable.h"
#include "TreeView/AssetTreeNode.h"
#include "TreeView/SAssetTableTreeView.h"

#define LOCTEXT_NAMESPACE "AssetManagementBrowser"

const FString SAssetAuditBrowser::SettingsIniSection = TEXT("AssetManagementBrowser");
const int32 SAssetAuditBrowser::MaxAssetsHistory = 10;

SAssetAuditBrowser::~SAssetAuditBrowser()
{
}

TSharedPtr<SWidget> SAssetAuditBrowser::OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets)
{
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/ true, Commands);

	MenuBuilder.BeginSection(TEXT("Asset"), NSLOCTEXT("ReferenceViewerSchema", "AssetSectionLabel", "Asset"));
	{
		MenuBuilder.AddMenuEntry(FGlobalEditorCommonCommands::Get().FindInContentBrowser);
		MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().OpenSelectedInAssetEditor);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("Load", "Load..."),
			LOCTEXT("LoadTooltip", "Loads selected assets into memory."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.OpenLevel"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SAssetAuditBrowser::LoadSelectedAssets),
				FCanExecuteAction::CreateSP(this, &SAssetAuditBrowser::IsAnythingSelected)
			)
		);

		MenuBuilder.AddMenuEntry(
			LOCTEXT("SaveSelectedAssets", "Save..."),
			LOCTEXT("SaveSelectedAssets_ToolTip", "Save the selected assets."),
			FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Save"),
			FUIAction(
				FExecuteAction::CreateSP(this, &SAssetAuditBrowser::SaveSelectedAssets),
				FCanExecuteAction::CreateSP(this, &SAssetAuditBrowser::IsAnythingSelectedAndLoaded)
			)
		);
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("References"), NSLOCTEXT("ReferenceViewerSchema", "ReferencesSectionLabel", "References"));
	{
		MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().ViewReferences);
		MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().ViewSizeMap);
	}
	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void SAssetAuditBrowser::FindInContentBrowser() const
{
	TArray<FAssetData> CurrentSelection = GetCurrentSelectionDelegate.Execute();
	if (CurrentSelection.Num() > 0)
	{
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		ContentBrowserModule.Get().SyncBrowserToAssets(CurrentSelection);
	}
}

bool SAssetAuditBrowser::IsAnythingSelected() const
{
	TArray<FAssetData> CurrentSelection = GetCurrentSelectionDelegate.Execute();
	return CurrentSelection.Num() > 0;
}

bool SAssetAuditBrowser::IsAnythingSelectedAndLoaded() const
{
	TArray<FAssetData> Assets = GetCurrentSelectionDelegate.Execute();
	TArray<UPackage*> Packages;
	GetSelectedPackages(Assets, Packages);

	// Don't offer option if none of the packages are loaded
	return Packages.Num() > 0;
}

void SAssetAuditBrowser::GetSelectedPackages(const TArray<FAssetData>& Assets, TArray<UPackage*>& OutPackages) const
{
	for (const FAssetData& AssetData : Assets)
	{
		UPackage* Package = FindPackage(nullptr, *AssetData.PackageName.ToString());

		if ( Package )
		{
			OutPackages.Add(Package);
		}
	}
}

void SAssetAuditBrowser::EditSelectedAssets() const
{
	TArray<FAssetData> Assets = GetCurrentSelectionDelegate.Execute();
	TArray<FSoftObjectPath> AssetPaths;

	for (FAssetData& AssetData : Assets)
	{
		AssetPaths.Add(AssetData.GetSoftObjectPath());
	}

	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorsForAssets(AssetPaths);
}

void SAssetAuditBrowser::OnRequestOpenAsset(const FAssetData& AssetData) const
{
	TArray<FSoftObjectPath> AssetPaths;
	AssetPaths.Add(AssetData.GetSoftObjectPath());

	GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorsForAssets(AssetPaths);
}

void SAssetAuditBrowser::SaveSelectedAssets() const
{
	TArray<FAssetData> Assets = GetCurrentSelectionDelegate.Execute();
	TArray<UPackage*> PackagesToSave;
	GetSelectedPackages(Assets, PackagesToSave);

	const bool bCheckDirty = false;
	const bool bPromptToSave = false;
	const FEditorFileUtils::EPromptReturnCode Return = FEditorFileUtils::PromptForCheckoutAndSave(PackagesToSave, bCheckDirty, bPromptToSave);
}

void SAssetAuditBrowser::FindReferencesForSelectedAssets() const
{
	TArray<FAssetData> Assets = GetCurrentSelectionDelegate.Execute();
	TArray<FAssetIdentifier> AssetIdentifiers;
	IAssetManagerEditorModule::ExtractAssetIdentifiersFromAssetDataList(Assets, AssetIdentifiers);

	if (AssetIdentifiers.Num() > 0)
	{
		IAssetManagerEditorModule::Get().OpenReferenceViewerUI(AssetIdentifiers);
	}
}

void SAssetAuditBrowser::ShowSizeMapForSelectedAssets() const
{
	TArray<FAssetData> Assets = GetCurrentSelectionDelegate.Execute();
	TArray<FAssetIdentifier> AssetIdentifiers;
	IAssetManagerEditorModule::ExtractAssetIdentifiersFromAssetDataList(Assets, AssetIdentifiers);

	if (AssetIdentifiers.Num() > 0)
	{
		IAssetManagerEditorModule::Get().OpenSizeMapUI(AssetIdentifiers);
	}
}

void SAssetAuditBrowser::LoadSelectedAssets() const
{
	TArray<FAssetData> Assets = GetCurrentSelectionDelegate.Execute();
	for (FAssetData& AssetData : Assets)
	{
		AssetData.GetAsset();
	}
}

bool SAssetAuditBrowser::CanShowColumnForAssetRegistryTag(FName AssetType, FName TagName) const
{
	return !AssetRegistryTagsToIgnore.Contains(TagName);
}

FString SAssetAuditBrowser::GetStringValueForCustomColumn(FAssetData& AssetData, FName ColumnName) const
{
	FString OutValue;
	EditorModule->GetStringValueForCustomColumn(AssetData, ColumnName, OutValue);
	return OutValue;
}

FText SAssetAuditBrowser::GetDisplayTextForCustomColumn(FAssetData& AssetData, FName ColumnName) const
{
	FText OutValue;
	EditorModule->GetDisplayTextForCustomColumn(AssetData, ColumnName, OutValue);
	return OutValue;
}

void SAssetAuditBrowser::OnGetCustomSourceAssets(const FARFilter& Filter, TArray<FAssetData>& OutAssets) const
{
	if (Filter.PackageNames.Contains(IAssetManagerEditorModule::ChunkFakeAssetDataPackageName) && CurrentRegistrySource)
	{
		IAssetManagerEditorModule::Get().GetCurrentRegistrySource(true);

		for (const TPair<int32, FAssetManagerChunkInfo>& Pair : CurrentRegistrySource->ChunkAssignments)
{
			OutAssets.Add(IAssetManagerEditorModule::CreateFakeAssetDataFromChunkId(Pair.Key));
		}
	}
}

void SAssetAuditBrowser::Construct(const FArguments& InArgs)
{
	check(UAssetManager::IsInitialized());

	IAssetManagerEditorModule& ManagerEditorModule = IAssetManagerEditorModule::Get();
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistry = &AssetRegistryModule.Get();
	AssetManager = &UAssetManager::Get();
	EditorModule = &IAssetManagerEditorModule::Get();

	TArray<const FAssetManagerEditorRegistrySource*> AvailableSources;
	ManagerEditorModule.GetAvailableRegistrySources(AvailableSources);

	for (const FAssetManagerEditorRegistrySource* ValidSource : AvailableSources)
	{
		SourceComboList.Add(MakeShared<FString>(ValidSource->SourceName));
	}

	CurrentRegistrySource = ManagerEditorModule.GetCurrentRegistrySource(false);

	Commands = MakeShareable(new FUICommandList());
	Commands->MapAction(FGlobalEditorCommonCommands::Get().FindInContentBrowser, FUIAction(
		FExecuteAction::CreateSP(this, &SAssetAuditBrowser::FindInContentBrowser),
		FCanExecuteAction::CreateSP(this, &SAssetAuditBrowser::IsAnythingSelected)
		));

	Commands->MapAction(FAssetManagerEditorCommands::Get().OpenSelectedInAssetEditor, FUIAction(
		FExecuteAction::CreateSP(this, &SAssetAuditBrowser::EditSelectedAssets),
		FCanExecuteAction::CreateSP(this, &SAssetAuditBrowser::IsAnythingSelected)
	));

	Commands->MapAction(FAssetManagerEditorCommands::Get().ViewReferences, FUIAction(
		FExecuteAction::CreateSP(this, &SAssetAuditBrowser::FindReferencesForSelectedAssets),
		FCanExecuteAction::CreateSP(this, &SAssetAuditBrowser::IsAnythingSelected)
	));

	Commands->MapAction(FAssetManagerEditorCommands::Get().ViewSizeMap, FUIAction(
		FExecuteAction::CreateSP(this, &SAssetAuditBrowser::ShowSizeMapForSelectedAssets),
		FCanExecuteAction::CreateSP(this, &SAssetAuditBrowser::IsAnythingSelected)
	));

	CurrentAssetHistoryIndex = 0;
	AssetHistory.Add(TSet<FName>());

	FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>(TEXT("ContentBrowser"));

	// Configure filter for asset picker
	FAssetPickerConfig Config;
	Config.InitialAssetViewType = EAssetViewType::Column;
	Config.bAddFilterUI = true;
	Config.bShowPathInColumnView = true;
	Config.bSortByPathInColumnView = true;
	Config.bCanShowClasses = false;

	// Configure response to click and double-click
	Config.OnAssetDoubleClicked = FOnAssetDoubleClicked::CreateSP(this, &SAssetAuditBrowser::OnRequestOpenAsset);
	Config.OnGetAssetContextMenu = FOnGetAssetContextMenu::CreateSP(this, &SAssetAuditBrowser::OnGetAssetContextMenu);
	Config.OnAssetTagWantsToBeDisplayed = FOnShouldDisplayAssetTag::CreateSP(this, &SAssetAuditBrowser::CanShowColumnForAssetRegistryTag);
	Config.OnGetCustomSourceAssets = FOnGetCustomSourceAssets::CreateSP(this, &SAssetAuditBrowser::OnGetCustomSourceAssets);
	Config.SyncToAssetsDelegates.Add(&SyncToAssetsDelegate);
	Config.OnShouldFilterAsset = FOnShouldFilterAsset::CreateSP(this, &SAssetAuditBrowser::HandleFilterAsset);
	Config.OnAssetSelected = FOnAssetSelected::CreateSP(this, &SAssetAuditBrowser::OnAssetSelected);
	Config.GetCurrentSelectionDelegates.Add(&GetCurrentSelectionDelegate);
	Config.SetFilterDelegates.Add(&SetFilterDelegate);
	Config.bFocusSearchBoxWhenOpened = false;

	Config.SaveSettingsName = SettingsIniSection;

	// Hide path and type by default
	Config.HiddenColumnNames.Add(TEXT("Class"));
	Config.HiddenColumnNames.Add(TEXT("Path"));
	// Hide item disk size since we display a custom version here
	Config.HiddenColumnNames.Add(ContentBrowserItemAttributes::ItemDiskSize.ToString());

	// These are generally not used so hide them by default.
	Config.HiddenColumnNames.Add(IAssetManagerEditorModule::ManagedDiskSizeName.ToString());
	Config.HiddenColumnNames.Add(IAssetManagerEditorModule::TotalUsageName.ToString());

	// Add custom columns
	Config.CustomColumns.Emplace(FPrimaryAssetId::PrimaryAssetTypeTag, LOCTEXT("AssetType", "Primary Type"), LOCTEXT("AssetTypeTooltip", "Primary Asset Type of this asset, if set"), UObject::FAssetRegistryTag::TT_Alphabetical, FOnGetCustomAssetColumnData::CreateSP(this, &SAssetAuditBrowser::GetStringValueForCustomColumn), FOnGetCustomAssetColumnDisplayText::CreateSP(this, &SAssetAuditBrowser::GetDisplayTextForCustomColumn));
	Config.CustomColumns.Emplace(FPrimaryAssetId::PrimaryAssetNameTag, LOCTEXT("AssetName", "Primary Name"), LOCTEXT("AssetNameTooltip", "Primary Asset Name of this asset, if set"), UObject::FAssetRegistryTag::TT_Alphabetical, FOnGetCustomAssetColumnData::CreateSP(this, &SAssetAuditBrowser::GetStringValueForCustomColumn), FOnGetCustomAssetColumnDisplayText::CreateSP(this, &SAssetAuditBrowser::GetDisplayTextForCustomColumn));
	Config.CustomColumns.Emplace(IAssetManagerEditorModule::ManagedDiskSizeName, LOCTEXT("ManagedDiskSize", "Disk Size"), LOCTEXT("ManagedDiskSizeTooltip", "Total disk space used by both this and all managed assets"), UObject::FAssetRegistryTag::TT_Numerical, FOnGetCustomAssetColumnData::CreateSP(this, &SAssetAuditBrowser::GetStringValueForCustomColumn), FOnGetCustomAssetColumnDisplayText::CreateSP(this, &SAssetAuditBrowser::GetDisplayTextForCustomColumn));
	Config.CustomColumns.Emplace(IAssetManagerEditorModule::DiskSizeName, LOCTEXT("DiskSize", "Exclusive Disk Size"), LOCTEXT("DiskSizeTooltip", "Size of saved file(s) on disk for this asset's package. If Asset Registry Writeback is enabled and a platform is selected, this will be the uncompressed size of the iostore chunks for the asset's package."), UObject::FAssetRegistryTag::TT_Numerical, FOnGetCustomAssetColumnData::CreateSP(this, &SAssetAuditBrowser::GetStringValueForCustomColumn), FOnGetCustomAssetColumnDisplayText::CreateSP(this, &SAssetAuditBrowser::GetDisplayTextForCustomColumn));
	Config.CustomColumns.Emplace(UE::AssetRegistry::Stage_ChunkCompressedSizeFName, LOCTEXT("StagedCompressedSize", "Staged Compressed Size"), LOCTEXT("StagedCompressedSizeTooltip", "Compressed size of iostore chunks for this asset's package. Only visible after staging."), UObject::FAssetRegistryTag::TT_Numerical, FOnGetCustomAssetColumnData::CreateSP(this, &SAssetAuditBrowser::GetStringValueForCustomColumn), FOnGetCustomAssetColumnDisplayText::CreateSP(this, &SAssetAuditBrowser::GetDisplayTextForCustomColumn));
	Config.CustomColumns.Emplace(IAssetManagerEditorModule::TotalUsageName, LOCTEXT("TotalUsage", "Total Usage"), LOCTEXT("TotalUsageTooltip", "Weighted count of Primary Assets that use this, higher usage means it's more likely to be in memory at runtime"), UObject::FAssetRegistryTag::TT_Numerical, FOnGetCustomAssetColumnData::CreateSP(this, &SAssetAuditBrowser::GetStringValueForCustomColumn), FOnGetCustomAssetColumnDisplayText::CreateSP(this, &SAssetAuditBrowser::GetDisplayTextForCustomColumn));
	Config.CustomColumns.Emplace(IAssetManagerEditorModule::CookRuleName, LOCTEXT("CookRule", "Cook Rule"), LOCTEXT("CookRuleTooltip", "Whether this asset will be cooked or not"), UObject::FAssetRegistryTag::TT_Alphabetical, FOnGetCustomAssetColumnData::CreateSP(this, &SAssetAuditBrowser::GetStringValueForCustomColumn), FOnGetCustomAssetColumnDisplayText::CreateSP(this, &SAssetAuditBrowser::GetDisplayTextForCustomColumn));
	Config.CustomColumns.Emplace(IAssetManagerEditorModule::ChunksName, LOCTEXT("Chunks", "Chunks"), LOCTEXT("ChunksTooltip", "List of chunks this will be added to when cooked"), UObject::FAssetRegistryTag::TT_Alphabetical, FOnGetCustomAssetColumnData::CreateSP(this, &SAssetAuditBrowser::GetStringValueForCustomColumn), FOnGetCustomAssetColumnDisplayText::CreateSP(this, &SAssetAuditBrowser::GetDisplayTextForCustomColumn));

	// Ignore these tags as we added them as custom columns
	AssetRegistryTagsToIgnore.Add(FPrimaryAssetId::PrimaryAssetTypeTag);
	AssetRegistryTagsToIgnore.Add(FPrimaryAssetId::PrimaryAssetNameTag);

	// Ignore blueprint tags
	AssetRegistryTagsToIgnore.Add(FBlueprintTags::ParentClassPath);
	AssetRegistryTagsToIgnore.Add(FBlueprintTags::BlueprintType);
	AssetRegistryTagsToIgnore.Add(FBlueprintTags::NumReplicatedProperties);
	AssetRegistryTagsToIgnore.Add(FBlueprintTags::NativeParentClassPath);
	AssetRegistryTagsToIgnore.Add(FBlueprintTags::IsDataOnly);
	AssetRegistryTagsToIgnore.Add(FBlueprintTags::NumNativeComponents);
	AssetRegistryTagsToIgnore.Add(FBlueprintTags::NumBlueprintComponents);

	static const FName DefaultForegroundName("DefaultForeground");

	TSharedRef< SMenuAnchor > BackMenuAnchorPtr = SNew(SMenuAnchor)
		.Placement(MenuPlacement_BelowAnchor)
		.OnGetMenuContent(this, &SAssetAuditBrowser::CreateHistoryMenu, true)
		[
			SNew(SButton)
			.OnClicked(this, &SAssetAuditBrowser::OnGoBackInHistory)
			.ForegroundColor(FAppStyle::GetSlateColor(DefaultForegroundName))
			.ButtonStyle(FAppStyle::Get(), "FlatButton")
			.ContentPadding(FMargin(1, 0))
			.IsEnabled(this, &SAssetAuditBrowser::CanStepBackwardInHistory)
			.ToolTipText(LOCTEXT("Backward_Tooltip", "Step backward in the asset history. Right click to see full history."))
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(FText::FromString(FString(TEXT("\xf060"))) /*fa-arrow-left*/)
			]
		];

	TSharedRef< SMenuAnchor > FwdMenuAnchorPtr = SNew(SMenuAnchor)
		.Placement(MenuPlacement_BelowAnchor)
		.OnGetMenuContent(this, &SAssetAuditBrowser::CreateHistoryMenu, false)
		[
			SNew(SButton)
			.OnClicked(this, &SAssetAuditBrowser::OnGoForwardInHistory)
			.ForegroundColor(FAppStyle::GetSlateColor(DefaultForegroundName))
			.ButtonStyle(FAppStyle::Get(), "FlatButton")
			.ContentPadding(FMargin(1, 0))
			.IsEnabled(this, &SAssetAuditBrowser::CanStepForwardInHistory)
			.ToolTipText(LOCTEXT("Forward_Tooltip", "Step forward in the asset history. Right click to see full history."))
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
				.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
				.Text(FText::FromString(FString(TEXT("\xf061"))) /*fa-arrow-right*/)
			]
		];

	this->ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SBorder)
			.Visibility(this, &SAssetAuditBrowser::GetHistoryVisibility)
			.Padding(FMargin(3))
			.BorderImage( FAppStyle::GetBrush("ToolPanel.GroupBorder") )
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Fill)
				[
					SNew(SHorizontalBox)
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SBorder)
						.OnMouseButtonDown(this, &SAssetAuditBrowser::OnMouseDownHistory, TWeakPtr<SMenuAnchor>(BackMenuAnchorPtr))
						.BorderImage( FAppStyle::GetBrush("NoBorder") )
						[
							BackMenuAnchorPtr
						]
					]

					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SBorder)
						.OnMouseButtonDown(this, &SAssetAuditBrowser::OnMouseDownHistory, TWeakPtr<SMenuAnchor>(FwdMenuAnchorPtr))
						.BorderImage( FAppStyle::GetBrush("NoBorder") )
						[
							FwdMenuAnchorPtr
						]
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.Text(LOCTEXT("ClearAssets", "Clear Assets"))
						.OnClicked(this, &SAssetAuditBrowser::ClearAssets)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						IAssetManagerEditorModule::MakePrimaryAssetTypeSelector(
							FOnGetPrimaryAssetDisplayText::CreateLambda([] { return LOCTEXT("AddAssetsOfType", "Add Primary Asset Type"); }),
							FOnSetPrimaryAssetType::CreateSP(this, &SAssetAuditBrowser::AddAssetsOfType),
							false, true)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SComboButton)
						.MenuContent()
						[
							CreateClassPicker()
						]
						.ButtonContent()
						[
							SNew(STextBlock)
							.Text(LOCTEXT("AddAssetClass", "Add Asset Class"))
						]
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						IAssetManagerEditorModule::MakePrimaryAssetIdSelector(
							FOnGetPrimaryAssetDisplayText::CreateLambda([] { return LOCTEXT("AddManagedAssets", "Add Managed Assets"); }),
							FOnSetPrimaryAssetId::CreateSP(this, &SAssetAuditBrowser::AddManagedAssets),
							false)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						IAssetManagerEditorModule::MakePrimaryAssetTypeSelector(
							FOnGetPrimaryAssetDisplayText::CreateLambda([] { return LOCTEXT("AddManagedAssetsOfType", "Add Managed Assets of Type"); }),
							FOnSetPrimaryAssetType::CreateSP(this, &SAssetAuditBrowser::AddManagedAssetsOfType),
							false, false)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.Text(LOCTEXT("AddChunks", "Add Chunks"))
						.OnClicked(this, &SAssetAuditBrowser::AddChunks)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						.Text(LOCTEXT("RefreshAssets", "Refresh"))
						.OnClicked(this, &SAssetAuditBrowser::RefreshAssets)
					]

					+SHorizontalBox::Slot()
					.FillWidth(1.0f)
					.HAlign(HAlign_Fill)
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Fill)
						.FillWidth(1.0f)
						[
							SNew(SHorizontalBox)
							+ SHorizontalBox::Slot()
							.Padding(6.0f)
							.VAlign(VAlign_Center)
							.HAlign(HAlign_Left)
							.FillWidth(1.0f)
							[
								SAssignNew(RegistrySourceTimeText, STextBlock)
								.ToolTipText(LOCTEXT("RegistryTime_TT", "If viewing cooked data, the timestamp for the AssetRegistry.bin file. Changes to assets are not picked up until a new cook. Otherwise, viewing preview data from the editor."))
							]
							+ SHorizontalBox::Slot()
							.Padding(6.0f)
							.AutoWidth()
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Center)
							[
								SNew(STextBlock)
								.ToolTipText(LOCTEXT("Platform_Tooltip", "Select which platform to display data for. Platforms are only available if a cooked AssetRegistry.bin is available in Saved/Cooked/Platform or Build/Platform."))
								.Text(LOCTEXT("PlatformLabel", "Selected Platform: "))
							]
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.HAlign(HAlign_Right)
						[
							SNew(SComboBox<TSharedPtr<FString>>)
							.OptionsSource(&SourceComboList)
							.OnGenerateWidget(this, &SAssetAuditBrowser::GenerateSourceComboItem)
							.OnSelectionChanged(this, &SAssetAuditBrowser::HandleSourceComboChanged)
							[
								SNew(STextBlock)
								.Text(this, &SAssetAuditBrowser::GetSourceComboText)
							]
						]
					]
				]
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SNew(SBorder)
			.Padding(FMargin(3))
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				ContentBrowserModule.Get().CreateAssetPicker(Config)
			]
		]
	];

	// Refresh our data sources to make sure we are displaying up to date info otherwise the user will need to
	// hit "refresh" every time they open the dialog to be able to trust it
	ManagerEditorModule.RefreshRegistryData();
}

FReply SAssetAuditBrowser::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (Commands->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

FReply SAssetAuditBrowser::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	bool bIsDragSupported = false;

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (Operation.IsValid() && Operation->IsOfType<FAssetDragDropOp>())
	{
		bIsDragSupported = true;
	}

	return bIsDragSupported ? FReply::Handled() : FReply::Unhandled();
}

FReply SAssetAuditBrowser::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	bool bWasDropHandled = false;
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

	if (Operation.IsValid())
	{
		if (Operation->IsOfType<FAssetDragDropOp>())
		{
			const FAssetDragDropOp& DragDropOp = *StaticCastSharedPtr<FAssetDragDropOp>(Operation);

			AddAssetsToList(DragDropOp.GetAssets(), false);
			bWasDropHandled = true;
		}
	}

	return bWasDropHandled ? FReply::Handled() : FReply::Unhandled();
}

void SAssetAuditBrowser::AddAssetsToList(const TArray<FSoftObjectPath>& AssetsToView, bool bReplaceExisting)
{
	TArray<FName> AssetNames;

	for (const FSoftObjectPath& AssetToView : AssetsToView)
	{
		AssetNames.Add(FName(*AssetToView.GetLongPackageName()));
	}

	AddAssetsToList(AssetNames, bReplaceExisting);
}

void SAssetAuditBrowser::AddAssetsToList(const TArray<FAssetData>& AssetsToView, bool bReplaceExisting)
{
	TArray<FName> AssetNames;

	for (const FAssetData& AssetToView : AssetsToView)
	{
		AssetNames.Add(AssetToView.PackageName);
	}

	AddAssetsToList(AssetNames, bReplaceExisting);
}

void SAssetAuditBrowser::AddAssetsToList(const TArray<FAssetIdentifier>& AssetsToView, bool bReplaceExisting)
{
	TArray<FName> AssetNames;

	for (const FAssetIdentifier& AssetToView : AssetsToView)
	{
		FName PackageName = AssetToView.PackageName;
		if (!PackageName.IsNone())
		{
			AssetNames.AddUnique(PackageName);
		}
	}

	AddAssetsToList(AssetNames, bReplaceExisting);
}

void SAssetAuditBrowser::AddAssetsToList(const TArray<FName>& PackageNamesToView, bool bReplaceExisting)
{
	AssetHistory.Insert(TSet<FName>(), ++CurrentAssetHistoryIndex);
	TSet<FName>& AssetSet = AssetHistory[CurrentAssetHistoryIndex];

	if (!bReplaceExisting && CurrentAssetHistoryIndex > 0)
	{
		// Copy old set
		AssetSet.Append(AssetHistory[CurrentAssetHistoryIndex - 1]);
	}

	AssetSet.Append(PackageNamesToView);

	RefreshAssetView();
}

FReply SAssetAuditBrowser::OnMouseDownHistory( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TWeakPtr< SMenuAnchor > InMenuAnchor )
{
	if(MouseEvent.GetEffectingButton() == EKeys::RightMouseButton)
	{
		InMenuAnchor.Pin()->SetIsOpen(true);
		return FReply::Handled();
	}

	return FReply::Unhandled();
}

TSharedRef<SWidget> SAssetAuditBrowser::CreateHistoryMenu(bool bInBackHistory) const
{
	FMenuBuilder MenuBuilder(true, NULL);
	if(bInBackHistory)
	{
		int32 HistoryIdx = CurrentAssetHistoryIndex - 1;
		while( HistoryIdx >= 0 )
		{
			const FText DisplayName = FText::Format(LOCTEXT("HistoryStringFormat", "{0} assets"), FText::AsNumber(AssetHistory[HistoryIdx].Num()));

			MenuBuilder.AddMenuEntry(DisplayName, DisplayName, FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(const_cast<SAssetAuditBrowser*>(this), &SAssetAuditBrowser::GoToHistoryIndex, HistoryIdx)
				),
				NAME_None, EUserInterfaceActionType::Button);
			--HistoryIdx;
		}
	}
	else
	{
		int32 HistoryIdx = CurrentAssetHistoryIndex + 1;
		while( HistoryIdx < AssetHistory.Num() )
		{
			const FText DisplayName = FText::Format(LOCTEXT("HistoryStringFormat", "{0} assets"), FText::AsNumber(AssetHistory[HistoryIdx].Num()));

			MenuBuilder.AddMenuEntry(DisplayName, DisplayName, FSlateIcon(),
				FUIAction(
					FExecuteAction::CreateRaw(const_cast<SAssetAuditBrowser*>(this), &SAssetAuditBrowser::GoToHistoryIndex, HistoryIdx)
				),
				NAME_None, EUserInterfaceActionType::Button);
			++HistoryIdx;
		}
	}

	return MenuBuilder.MakeWidget();
}

bool SAssetAuditBrowser::CanStepBackwardInHistory() const
{
	if (CurrentAssetHistoryIndex > 0)
	{
		return true;
	}
	return false;
}

bool SAssetAuditBrowser::CanStepForwardInHistory() const
{
	if (CurrentAssetHistoryIndex < AssetHistory.Num() - 1)
	{
		return true;
	}

	return false;
}

FReply SAssetAuditBrowser::OnGoForwardInHistory()
{
	GoToHistoryIndex(CurrentAssetHistoryIndex + 1);
	return FReply::Handled();
}

FReply SAssetAuditBrowser::OnGoBackInHistory()
{
	GoToHistoryIndex(CurrentAssetHistoryIndex - 1);
	return FReply::Handled();
}

void SAssetAuditBrowser::GoToHistoryIndex(int32 InHistoryIdx)
{
	if(AssetHistory.IsValidIndex(InHistoryIdx))
	{
		CurrentAssetHistoryIndex = InHistoryIdx;
		RefreshAssetView();
	}
}

void SAssetAuditBrowser::RefreshAssetView()
{
	FARFilter Filter;

	// Add manual package list
	const TSet<FName>& AssetSet = AssetHistory[CurrentAssetHistoryIndex];

	for (FName PackageName : AssetSet)
	{
		Filter.PackageNames.Add(PackageName);
	}

	if (Filter.PackageNames.Num() == 0)
	{
		// Add a bad name to force it to display nothing
		Filter.PackageNames.Add("/Temp/FakePackageNameToMakeNothingShowUp");
	}

	SetFilterDelegate.Execute(Filter);
}

FReply SAssetAuditBrowser::ClearAssets()
{
	AddAssetsToList(TArray<FName>(), true);

	return FReply::Handled();
}

FReply SAssetAuditBrowser::AddChunks()
{
	TArray<FName> PackageNames;
	PackageNames.Add(IAssetManagerEditorModule::ChunkFakeAssetDataPackageName);

	AddAssetsToList(PackageNames, false);

	return FReply::Handled();
}

FReply SAssetAuditBrowser::RefreshAssets()
{
	// This will end up refreshing the UI indirectly
	IAssetManagerEditorModule::Get().RefreshRegistryData();

	return FReply::Handled();
}

void SAssetAuditBrowser::AddAssetsOfType(FPrimaryAssetType AssetType)
{
	if (AssetType.IsValid())
	{
		TArray<FSoftObjectPath> AssetArray;

		if (AssetType == IAssetManagerEditorModule::AllPrimaryAssetTypes)
		{
			TArray<FPrimaryAssetTypeInfo> TypeList;
			AssetManager->GetPrimaryAssetTypeInfoList(TypeList);

			for (const FPrimaryAssetTypeInfo& TypeInfo : TypeList)
			{
				AssetManager->GetPrimaryAssetPathList(TypeInfo.PrimaryAssetType, AssetArray);
			}
		}
		else
		{
			AssetManager->GetPrimaryAssetPathList(AssetType, AssetArray);
		}

		AddAssetsToList(AssetArray, false);
	}
}

void SAssetAuditBrowser::AddManagedAssets(FPrimaryAssetId AssetId)
{
	if (AssetId.IsValid())
	{
		TArray<FName> AssetPackageArray;

		AssetManager->GetManagedPackageList(AssetId, AssetPackageArray);

		AddAssetsToList(AssetPackageArray, false);
	}
}

void SAssetAuditBrowser::AddManagedAssetsOfType(FPrimaryAssetType AssetType)
{
	if (AssetType.IsValid())
	{
		TArray<FSoftObjectPath> AssetArray;
		AssetManager->GetPrimaryAssetPathList(AssetType, AssetArray);
		AddAssetsToList(AssetArray, false);

		TArray<FName> AssetPackageArray;
		TArray<FPrimaryAssetId> PrimaryAssetIds;
		AssetManager->GetPrimaryAssetIdList(AssetType, PrimaryAssetIds);
		for (const FPrimaryAssetId& PrimaryAssetId : PrimaryAssetIds)
		{
			// Calling GetManagedPackageList rather than AddManagedAssets to reduce UI update calls
			AssetManager->GetManagedPackageList(PrimaryAssetId, AssetPackageArray);
		}
		AddAssetsToList(AssetPackageArray, false);
	}
}

void SAssetAuditBrowser::AddAssetsOfClass(UClass* AssetClass)
{
	FSlateApplication::Get().DismissAllMenus();

	if (AssetClass)
	{
		TArray<FAssetData> FoundData;
		FARFilter AssetFilter;
		TSet<FTopLevelAssetPath> DerivedClassNames;
		bool bAssetClassIsBlueprint = AssetClass->IsChildOf(UBlueprintCore::StaticClass());

		AssetFilter.ClassPaths.Add(AssetClass->GetClassPathName());
		AssetFilter.bRecursiveClasses = true;

		if (!bAssetClassIsBlueprint)
		{
			// If we didn't search specifically for a Blueprint class, find the derived class and look for all blueprints of those derived classes
			AssetRegistry->GetDerivedClassNames(AssetFilter.ClassPaths, TSet<FTopLevelAssetPath>(), DerivedClassNames);
			AssetFilter.ClassPaths.Add(UBlueprintCore::StaticClass()->GetClassPathName());
		}

		if (AssetRegistry->GetAssets(AssetFilter, FoundData) && FoundData.Num() > 0)
		{
			TArray<FName> AssetPackageArray;

			for (FAssetData& AssetData : FoundData)
			{
				if (!bAssetClassIsBlueprint && AssetData.AssetClassPath.GetAssetName().ToString().EndsWith(TEXT("Blueprint")))
				{
					// This is a blueprint but the filter type wasn't, check the derived class list
					if (!AssetManager->IsAssetDataBlueprintOfClassSet(AssetData, DerivedClassNames))
					{
						continue;
					}
				}

				AssetPackageArray.Add(AssetData.PackageName);
			}

			AddAssetsToList(AssetPackageArray, false);
		}
	}
}

TSharedRef<SWidget> SAssetAuditBrowser::CreateClassPicker()
{
	FClassViewerInitializationOptions Options;
	Options.Mode = EClassViewerMode::ClassPicker;
	Options.DisplayMode = EClassViewerDisplayMode::ListView;
	Options.bShowObjectRootClass = true;
	Options.bShowNoneOption = false;

	// This will allow unloaded blueprints to be shown.
	Options.bShowUnloadedBlueprints = true;

	return FModuleManager::LoadModuleChecked<FClassViewerModule>("ClassViewer").CreateClassViewer(Options, FOnClassPicked::CreateSP(this, &SAssetAuditBrowser::AddAssetsOfClass));
}

EVisibility SAssetAuditBrowser::GetHistoryVisibility() const
{
	return EVisibility::Visible;
}

bool SAssetAuditBrowser::HandleFilterAsset(const FAssetData& InAssetData) const
{
	if (InAssetData.PackagePath == IAssetManagerEditorModule::PrimaryAssetFakeAssetDataPackagePath)
	{
		return false;
	}

	return !EditorModule->IsPackageInCurrentRegistrySource(InAssetData.PackageName);
}

void SAssetAuditBrowser::OnAssetSelected(const FAssetData& InAssetData)
{
}

TSharedRef<SWidget> SAssetAuditBrowser::GenerateSourceComboItem(TSharedPtr<FString> InItem)
{
	return SNew(STextBlock).Text(FText::FromString(*InItem));
}

void SAssetAuditBrowser::HandleSourceComboChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo)
{
	if (Item.IsValid())
	{
		// This will call back into this widget to refresh it
		EditorModule->SetCurrentRegistrySource(*Item);
	}
}

FText SAssetAuditBrowser::GetSourceComboText() const
{
	if (CurrentRegistrySource)
	{
		return FText::FromString(CurrentRegistrySource->SourceName);
	}

	return FText::GetEmpty();
}

void SAssetAuditBrowser::SetCurrentRegistrySource(const FAssetManagerEditorRegistrySource* RegistrySource)
{
	CurrentRegistrySource = RegistrySource;

	if (CurrentRegistrySource->bIsEditor)
	{
		RegistrySourceTimeText->SetText(LOCTEXT("AssetRegistryTimestampEditor", "Preview data"));
	}
	else
	{
		if (CurrentRegistrySource->SourceTimestamp.Len() == 0)
		{
			RegistrySourceTimeText->SetText(LOCTEXT("AssetRegistryTimestampError", "Unable to get AssetRegistry.bin timestamp (recook?)"));
		}
		else
		{
			RegistrySourceTimeText->SetText(FText::Format(LOCTEXT("AssetRegistryTimestampFormat", "AssetRegistry.bin from {0}"), FText::FromString(CurrentRegistrySource->SourceTimestamp)));
		}
	}

	// Refresh dropdown
	TArray<const FAssetManagerEditorRegistrySource*> AvailableSources;
	IAssetManagerEditorModule::Get().GetAvailableRegistrySources(AvailableSources);
	SourceComboList.Reset();

	for (const FAssetManagerEditorRegistrySource* ValidSource : AvailableSources)
	{
		SourceComboList.Add(MakeShared<FString>(ValidSource->SourceName));
	}

	RefreshAssetView();
}

void SAssetAuditBrowser::OnClose()
{
}

#undef LOCTEXT_NAMESPACE
