// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ContentBrowserDelegates.h"
#include "Widgets/SCompoundWidget.h"

struct FAssetIdentifier;
struct FAssetManagerEditorRegistrySource;

class FUICommandList;
class SMenuAnchor;
class STextBlock;

//////////////////////////////////////////////////////////////////////////
// SAssetAuditBrowser

class SAssetAuditBrowser : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SAssetAuditBrowser)
	{}

	SLATE_END_ARGS()
public:
	void Construct(const FArguments& InArgs);

	virtual ~SAssetAuditBrowser();

	/** Adds assets to current management view */
	void AddAssetsToList(const TArray<FAssetData>& AssetsToView, bool bReplaceExisting);
	void AddAssetsToList(const TArray<FSoftObjectPath>& AssetsToView, bool bReplaceExisting);
	void AddAssetsToList(const TArray<FAssetIdentifier>& AssetsToView, bool bReplaceExisting);
	void AddAssetsToList(const TArray<FName>& PackageNamesToView, bool bReplaceExisting);

	/** Called when the current registry source changes */
	void SetCurrentRegistrySource(const FAssetManagerEditorRegistrySource* RegistrySource);

	/** Called when tab is closed */
	void OnClose();

protected:
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/** Delegate that handles creation of context menu */
	TSharedPtr<SWidget> OnGetAssetContextMenu(const TArray<FAssetData>& SelectedAssets);

	/** Delegates for context menu */
	void FindInContentBrowser() const;
	bool IsAnythingSelected() const;
	bool IsAnythingSelectedAndLoaded() const;
	void OnRequestOpenAsset(const FAssetData& AssetData) const;

	void EditSelectedAssets() const;
	void SaveSelectedAssets() const;
	void FindReferencesForSelectedAssets() const;
	void ShowSizeMapForSelectedAssets() const;
	void LoadSelectedAssets() const;

	bool CanShowColumnForAssetRegistryTag(FName AssetType, FName TagName) const;
	FString GetStringValueForCustomColumn(FAssetData& AssetData, FName ColumnName) const;
	FText GetDisplayTextForCustomColumn(FAssetData& AssetData, FName ColumnName) const;
	void OnGetCustomSourceAssets(const FARFilter& Filter, TArray<FAssetData>& OutAssets) const;

	/** Populate supplied OutPackages with the packages for the supplied Assets array */
	void GetSelectedPackages(const TArray<FAssetData>& Assets, TArray<UPackage*>& OutPackages) const;
	void GetSelectedAssetIdentifiers(const TArray<FAssetData>& Assets, TArray<FAssetIdentifier>& OutAssetIdentifiers) const;

	/** Single step forward in history */
	FReply OnGoForwardInHistory();

	/** Single step back in history */
	FReply OnGoBackInHistory();

	/** Jumps immediately to an index in the history if valid */
	void GoToHistoryIndex(int32 InHistoryIdx);

	/** Returns TRUE if stepping backward in history is allowed */
	bool CanStepBackwardInHistory() const;

	/** Returns TRUE if stepping forward in history is allowed */
	bool CanStepForwardInHistory() const;

	/**
	 * Mouse down callback to display a history menu
	 *
	 * @param InMenuAnchor		This is the anchor the menu will use for positioning
	 */
	FReply OnMouseDownHistory( const FGeometry& MyGeometry, const FPointerEvent& MouseEvent, TWeakPtr< SMenuAnchor > InMenuAnchor );

	/** 
	 * Callback to create the history menu.
	 *
	 * @param bInBackHistory		TRUE if the back history is requested, FALSE if the forward history is
	 *
	 * @return						The menu widget displaying all available history
	 */
	TSharedRef<SWidget> CreateHistoryMenu(bool bInBackHistory) const;

	/** Returns visible when not in a Blueprint mode */
	EVisibility GetHistoryVisibility() const;

	/** Perform additional filtering */
	bool HandleFilterAsset(const FAssetData& InAssetData) const;

	void OnAssetSelected(const FAssetData& InAssetData);

	/** Button callbacks */
	FReply ClearAssets();
	FReply AddChunks();
	FReply RefreshAssets();

	/** Adds all assets of type */
	void AddAssetsOfType(FPrimaryAssetType AssetType);
	
	/** Adds all managed assets by an id */
	void AddManagedAssets(FPrimaryAssetId AssetId);

	/** Adds all managed assets by by type. Useful to see how much size a primary asset type is using. */
	void AddManagedAssetsOfType(FPrimaryAssetType AssetType);

	/** Adds all assets of a class */
	void AddAssetsOfClass(UClass* AssetClass);

	/** Generates a class picker combo button content */
	TSharedRef<SWidget> CreateClassPicker();

	/** Platform combo box */
	TSharedRef<SWidget> GenerateSourceComboItem(TSharedPtr<FString> InItem);
	void HandleSourceComboChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo);
	FText GetSourceComboText() const;

	/** Refresh the asset view with a new filter */
	void RefreshAssetView();

	/** Commands handled by this widget */
	TSharedPtr<FUICommandList> Commands;

	/** Set of tags to prevent creating details view columns for (infrequently used) */
	TSet<FName> AssetRegistryTagsToIgnore;

	/** List of asset sets to display in browser, current index sets what to display */
	TArray<TSet<FName>> AssetHistory;

	/** Current position in the asset history */
	int32 CurrentAssetHistoryIndex;

	/** List of valid registry sources */
	TArray<TSharedPtr<FString>> SourceComboList;

	/**
	* Text block where we put the timestamp for the asset registry so the user knows
	* if they are looking at super out of date data.
	*/
	TSharedPtr<STextBlock> RegistrySourceTimeText;

	/** Current TargetPlatform and registry state, may be null! */
	const FAssetManagerEditorRegistrySource* CurrentRegistrySource;

	/** Delegates to interact with asset view */
	FSyncToAssetsDelegate SyncToAssetsDelegate;
	FGetCurrentSelectionDelegate GetCurrentSelectionDelegate;
	FSetARFilterDelegate SetFilterDelegate;

	/** Max asset sets to save in history */
	static const int32 MaxAssetsHistory;

	/** The section of EditorPerProjectUserSettings in which to save settings */
	static const FString SettingsIniSection;

	/** Cached interfaces */
	class IAssetRegistry* AssetRegistry;
	class UAssetManager* AssetManager;
	class IAssetManagerEditorModule* EditorModule;
};
