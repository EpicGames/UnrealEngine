// Copyright Epic Games, Inc. All Rights Reserved.

#include "SSizeMap.h"
#include "Engine/AssetManager.h"
#include "Editor.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetThumbnail.h"
#include "ClassIconFinder.h"
#include "Framework/Views/TableViewMetadata.h"
#include "Widgets/Input/SButton.h"
#include "Framework/Commands/UICommandList.h"

#include "Misc/PackageName.h"
#include "Toolkits/GlobalEditorCommonCommands.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "AssetManagerEditorModule.h"
#include "AssetManagerEditorCommands.h"
#include "DragAndDrop/AssetDragDropOp.h"
#include "CollectionManagerModule.h"
#include "ICollectionContainer.h"
#include "ICollectionManager.h"
#include "ICollectionSource.h"
#include "Misc/ScopedSlowTask.h"
#include "STreeMap.h"
#include "Subsystems/AssetEditorSubsystem.h"
#include "Widgets/Input/SComboBox.h"
#include "Engine/ExternalObjectAndActorDependencyGatherer.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Interfaces/IPluginManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SSizeMap)

#define LOCTEXT_NAMESPACE "SizeMap"

SSizeMap::SSizeMap()
	: TreeMapWidget(nullptr),
	RootTreeMapNode(new FTreeMapNodeData()),

	// @todo sizemap: Hard-coded thumbnail pool size.  Not a big deal, but ideally move the constants elsewhere
	AssetThumbnailPool(new FAssetThumbnailPool(1024))
{
}

SSizeMap::~SSizeMap()
{
	AssetThumbnailPool.Reset();
}

FReply SSizeMap::OnZoomOut()
{
	if (TreeMapWidget->ZoomOut())
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

bool SSizeMap::CanZoomOut() const
{
	return TreeMapWidget->CanZoomOut();
}

TSharedRef<SWidget> SSizeMap::GenerateSizeTypeComboItem(TSharedPtr<FName> InItem) const
{
	return SNew(STextBlock).Text(GetSizeTypeText(*InItem.Get()));
}

void SSizeMap::HandleSizeTypeComboChanged(TSharedPtr<FName> Item, ESelectInfo::Type SelectInfo)
{
	FSlateApplication::Get().DismissAllMenus();

	if (USizeMapSettings* SizeMapSettings = GetMutableDefault<USizeMapSettings>())
	{
		SizeMapSettings->SizeType = *Item.Get();
		SizeMapSettings->SaveConfig();
	}

	RefreshMap();
}

FText SSizeMap::GetSizeTypeComboText() const
{
	return GetSizeTypeText(GetDefault<USizeMapSettings>()->SizeType);
}

FText SSizeMap::GetSizeTypeText(FName SizeType) const
{
	if (SizeType == IAssetManagerEditorModule::ResourceSizeName)
	{
		return LOCTEXT("ResourceSize", "Memory Size");
	}
	else if (SizeType == IAssetManagerEditorModule::DiskSizeName)
	{
		return LOCTEXT("DiskSize", "Disk Size");
	}
	return FText::GetEmpty();
}

bool SSizeMap::IsSizeTypeEnabled() const
{
	return CurrentRegistrySource && CurrentRegistrySource->bIsEditor;
}

FText SSizeMap::GetOverviewText() const
{
	return OverviewText;
}

TSharedRef<SWidget> SSizeMap::GenerateDependencyTypeComboItem(TSharedPtr<ESizeMapDependencyType> InItem) const
{
	return SNew(STextBlock).Text(GetDependencyTypeText(*InItem.Get()));
}

void SSizeMap::HandleDependencyTypeComboChanged(TSharedPtr<ESizeMapDependencyType> Item, ESelectInfo::Type SelectInfo)
{
	if (USizeMapSettings* SizeMapSettings = GetMutableDefault<USizeMapSettings>())
	{
		SizeMapSettings->DependencyType = *Item.Get();
		SizeMapSettings->SaveConfig();
	}

	RefreshMap();
}

FText SSizeMap::GetDependencyTypeComboText() const
{
	return GetDependencyTypeText(GetDefault<USizeMapSettings>()->DependencyType);
}

FText SSizeMap::GetDependencyTypeText(ESizeMapDependencyType DependencyType) const
{
	switch (DependencyType)
	{
	case ESizeMapDependencyType::All:
		return LOCTEXT("DependencyType_All", "All");
	case ESizeMapDependencyType::Game:
		return LOCTEXT("DependencyType_Game", "Game");
	case ESizeMapDependencyType::EditorOnly:
		return LOCTEXT("DependencyType_Editor", "Editor Only");
	default:
		return FText::GetEmpty();
	}
}

TSharedRef<SWidget> SSizeMap::GenerateSourceComboItem(TSharedPtr<FString> InItem) const
{
	return SNew(STextBlock).Text(FText::FromString(*InItem));
}

void SSizeMap::HandleSourceComboChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo)
{
	if (Item.IsValid())
	{
		EditorModule->SetCurrentRegistrySource(*Item);
	}
}

FText SSizeMap::GetSourceComboText() const
{
	if (CurrentRegistrySource)
	{
		return FText::FromString(CurrentRegistrySource->SourceName);
	}
	return FText::GetEmpty();
}

TSharedRef<SWidget> SSizeMap::GeneratePluginComboItem(TSharedPtr<FString> InItem) const
{
	const FText DisplayText = InItem->IsEmpty()
		? LOCTEXT("PluginFilter_None", "None")
		: FText::FromString(*InItem);
	return SNew(STextBlock).Text(DisplayText);
}

void SSizeMap::HandlePluginComboChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo)
{
	if (Item.IsValid())
	{
		CurrentPluginSelection = *Item;
		if (!CurrentPluginSelection.IsEmpty())
		{
			SetRootAssetIdentifiersFromPlugin(CurrentPluginSelection);
		}
	}
}

FText SSizeMap::GetPluginComboText() const
{
	return CurrentPluginSelection.IsEmpty()
		? LOCTEXT("PluginFilter_None", "None")
		: FText::FromString(CurrentPluginSelection);
}

void SSizeMap::SetRootAssetIdentifiersFromPlugin(const FString& PluginName)
{
	TSet<FString> AllPluginNames;
	// Only support a single plugin being collected right now. Too many will overload the display.
	AllPluginNames.Add(PluginName);

	// Gather all asset package names from those plugins via the editor asset registry
	IPluginManager& PluginManager = IPluginManager::Get();
	FARFilter Filter;
	Filter.bRecursivePaths = true;
	for (const FString& Name : AllPluginNames)
	{
		TSharedPtr<IPlugin> Plugin = PluginManager.FindPlugin(Name);
		if (!Plugin.IsValid() || !Plugin->CanContainContent())
		{
			continue;
		}
		Filter.PackagePaths.Add(FName(FString::Printf(TEXT("/%s/"), *Plugin->GetName())));
	}

	TArray<FAssetData> FoundAssets;
	CurrentRegistrySource->GetAssets(Filter, FoundAssets);
	TSet<FAssetIdentifier> AssetIdentifiers;
	AssetIdentifiers.Reserve(FoundAssets.Num());
	for (const FAssetData& Asset : FoundAssets)
	{
		AssetIdentifiers.Add(FAssetIdentifier(Asset.PackageName));
	}

	// Replace existing roots — SetRootAssetIdentifiers clears before setting
	SetRootAssetIdentifiers(AssetIdentifiers.Array());
}

void SSizeMap::Construct(const FArguments& InArgs)
{
	IAssetManagerEditorModule& ManagerEditorModule = IAssetManagerEditorModule::Get();
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked<FAssetRegistryModule>(TEXT("AssetRegistry"));
	AssetRegistry = &AssetRegistryModule.Get();
	AssetManager = &UAssetManager::Get();
	EditorModule = &IAssetManagerEditorModule::Get();

	SizeTypeComboList.Add(MakeShared<FName>(IAssetManagerEditorModule::DiskSizeName));
	SizeTypeComboList.Add(MakeShared<FName>(IAssetManagerEditorModule::ResourceSizeName));
	
	DependencyTypeComboList.Add(MakeShared<ESizeMapDependencyType>(ESizeMapDependencyType::All));
	DependencyTypeComboList.Add(MakeShared<ESizeMapDependencyType>(ESizeMapDependencyType::Game));
	DependencyTypeComboList.Add(MakeShared<ESizeMapDependencyType>(ESizeMapDependencyType::EditorOnly));

	// Populate plugin picker list — all enabled plugins that can contain content, sorted by name
	PluginComboList.Add(MakeShared<FString>(TEXT(""))); // "None" sentinel (empty string)
	{
		IPluginManager& PluginManager = IPluginManager::Get();
		TArray<TSharedRef<IPlugin>> EnabledPlugins = PluginManager.GetEnabledPluginsWithContent();
		EnabledPlugins.Sort([](const TSharedRef<IPlugin>& A, const TSharedRef<IPlugin>& B)
		{
			return A->GetName() < B->GetName();
		});
		for (const TSharedRef<IPlugin>& Plugin : EnabledPlugins)
		{
			PluginComboList.Add(MakeShared<FString>(Plugin->GetName()));
		}
	}

	Commands = MakeShareable(new FUICommandList());

	Commands->MapAction(
		FGlobalEditorCommonCommands::Get().FindInContentBrowser,
		FExecuteAction::CreateSP(this, &SSizeMap::FindInContentBrowser),
		FCanExecuteAction::CreateSP(this, &SSizeMap::IsAnythingSelected));

	Commands->MapAction(
		FAssetManagerEditorCommands::Get().OpenSelectedInAssetEditor,
		FExecuteAction::CreateSP(this, &SSizeMap::EditSelectedAssets),
		FCanExecuteAction::CreateSP(this, &SSizeMap::IsAnythingSelected));

	Commands->MapAction(
		FAssetManagerEditorCommands::Get().ViewReferences,
		FExecuteAction::CreateSP(this, &SSizeMap::FindReferencesForSelectedAssets),
		FCanExecuteAction::CreateSP(this, &SSizeMap::IsAnythingSelected));

	Commands->MapAction(
		FAssetManagerEditorCommands::Get().ViewAssetAudit,
		FExecuteAction::CreateSP(this, &SSizeMap::ShowAssetAuditForSelectedAssets),
		FCanExecuteAction::CreateSP(this, &SSizeMap::IsAnythingSelected));

	Commands->MapAction(
		FAssetManagerEditorCommands::Get().AuditReferencedObjects,
		FExecuteAction::CreateSP(this, &SSizeMap::ShowAssetAuditForReferencedAssets));

	Commands->MapAction(
		FAssetManagerEditorCommands::Get().MakeLocalCollectionWithDependencies,
		FExecuteAction::CreateSP(this, &SSizeMap::MakeCollectionWithDependencies, ECollectionShareType::CST_Local),
		FCanExecuteAction::CreateSP(this, &SSizeMap::CanMakeCollectionWithDependencies, ECollectionShareType::CST_Local));

	Commands->MapAction(
		FAssetManagerEditorCommands::Get().MakePrivateCollectionWithDependencies,
		FExecuteAction::CreateSP(this, &SSizeMap::MakeCollectionWithDependencies, ECollectionShareType::CST_Private),
		FCanExecuteAction::CreateSP(this, &SSizeMap::CanMakeCollectionWithDependencies, ECollectionShareType::CST_Private));

	Commands->MapAction(
		FAssetManagerEditorCommands::Get().MakeSharedCollectionWithDependencies,
		FExecuteAction::CreateSP(this, &SSizeMap::MakeCollectionWithDependencies, ECollectionShareType::CST_Shared),
		FCanExecuteAction::CreateSP(this, &SSizeMap::CanMakeCollectionWithDependencies, ECollectionShareType::CST_Shared));

	Commands->MapAction(
		FAssetManagerEditorCommands::Get().CopyPaths,
		FExecuteAction::CreateSP(this, &SSizeMap::CopyPackageName),
		FCanExecuteAction::CreateSP(this, &SSizeMap::IsAnythingSelected));

	CurrentRegistrySource = EditorModule->GetCurrentRegistrySource(false);
	bMemorySizeCached = false;

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(SButton)
				.OnClicked(this, &SSizeMap::OnZoomOut)
				.ForegroundColor(FAppStyle::GetSlateColor("DefaultForeground"))
				.ButtonStyle(FAppStyle::Get(), "FlatButton")
				.ContentPadding(FMargin(1, 0))
				.IsEnabled(this, &SSizeMap::CanZoomOut)
				.ToolTipText(LOCTEXT("Backward_Tooltip", "Zoom Out, Mouse Wheel also works"))
				[
					SNew(STextBlock)
					.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
					.Font(FAppStyle::Get().GetFontStyle("FontAwesome.11"))
					.Text(FText::FromString(FString(TEXT("\xf060"))) /*fa-arrow-left*/)
				]
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.HAlign(HAlign_Fill)
			.VAlign(VAlign_Center)
			.Padding(4.f)
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "ContentBrowser.TopBar.Font")
				.Text(this, &SSizeMap::GetOverviewText)
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(2.f)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("DependencyType_Label", "Dependencies to Display:"))
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(2.f)
			.AutoWidth()
			[
				SAssignNew(DependencyTypeComboBoxWidget, SComboBox<TSharedPtr<ESizeMapDependencyType>>)
				.OptionsSource(&DependencyTypeComboList)
				.OnGenerateWidget(this, &SSizeMap::GenerateDependencyTypeComboItem)
				.OnSelectionChanged(this, &SSizeMap::HandleDependencyTypeComboChanged)
				.ToolTipText(LOCTEXT("DependencyType_Tooltip", "Changes which dependencies are considered in the size calculation"))
				[
					SNew(STextBlock)
					.Text(this, &SSizeMap::GetDependencyTypeComboText)
				]
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(2.f)
			.AutoWidth()
			[
				SNew(STextBlock)
				.ToolTipText(LOCTEXT("Platform_Tooltip", "Select which platform to display size data for. Platforms are only available if a cooked DevelopmentAssetRegistry.bin exists in Saved/Cooked/[Platform]/Metadata."))
				.Text(LOCTEXT("PlatformLabel", "Selected Platform: "))
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(2.f)
			.AutoWidth()
			[
				SAssignNew(SourceComboBoxWidget, SComboBox<TSharedPtr<FString>>)
				.OptionsSource(&SourceComboList)
				.OnGenerateWidget(this, &SSizeMap::GenerateSourceComboItem)
				.OnSelectionChanged(this, &SSizeMap::HandleSourceComboChanged)
				[
					SNew(STextBlock)
					.Text(this, &SSizeMap::GetSourceComboText)
				]
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(2.f)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SizeToDisplay", "Size to Display:"))
			]
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			.VAlign(VAlign_Center)
			.Padding(2.f)
			.AutoWidth()
			[
				SAssignNew(SizeTypeComboBoxWidget, SComboBox<TSharedPtr<FName>>)
				.OptionsSource(&SizeTypeComboList)
				.OnGenerateWidget(this, &SSizeMap::GenerateSizeTypeComboItem)
				.OnSelectionChanged(this, &SSizeMap::HandleSizeTypeComboChanged)
				.IsEnabled(this, &SSizeMap::IsSizeTypeEnabled)
				.ToolTipText(LOCTEXT("SizeType_Tooltip", "Change to display disk size or memory size, memory size is only available for editor data"))
				[
					SNew(STextBlock)
					.Text(this, &SSizeMap::GetSizeTypeComboText)
				]
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.f)
			[
				SNew(STextBlock)
				.Text(LOCTEXT("PluginFilter_Label", "Plugin Filter:"))
				.ToolTipText(LOCTEXT("PluginFilter_Tooltip", "Select a plugin to use as the size map root. All assets from the selected plugin will be shown."))
			]
			+SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			.Padding(2.f)
			[
				SAssignNew(PluginComboBoxWidget, SComboBox<TSharedPtr<FString>>)
				.OptionsSource(&PluginComboList)
				.OnGenerateWidget(this, &SSizeMap::GeneratePluginComboItem)
				.OnSelectionChanged(this, &SSizeMap::HandlePluginComboChanged)
				.ToolTipText(LOCTEXT("PluginFilter_Combo_Tooltip", "Select a plugin. All assets from this plugin will replace the current size map roots."))
				[
					SNew(STextBlock)
					.Text(this, &SSizeMap::GetPluginComboText)
				]
			]
		]
		+SVerticalBox::Slot()
		.FillHeight(1.f)
		[
			SAssignNew(TreeMapWidget, STreeMap, RootTreeMapNode.ToSharedRef(), nullptr)
				.OnTreeMapNodeRightClicked(this, &SSizeMap::OnTreeMapNodeRightClicked)
		]
	];

	// Refresh registry data to discover any available platform registries (e.g. DevelopmentAssetRegistry.bin)
	EditorModule->RefreshRegistryData();
}

FReply SSizeMap::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (Commands.IsValid() && Commands->ProcessCommandBindings(InKeyEvent))
	{
		return FReply::Handled();
	}
	return FReply::Unhandled();
}

FReply SSizeMap::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	bool bIsDragSupported = false;

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (Operation.IsValid() && Operation->IsOfType<FAssetDragDropOp>())
	{
		bIsDragSupported = true;
	}

	return bIsDragSupported ? FReply::Handled() : FReply::Unhandled();
}

FReply SSizeMap::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	bool bWasDropHandled = false;
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();

	if (Operation.IsValid())
	{
		if (Operation->IsOfType<FAssetDragDropOp>())
		{
			const FAssetDragDropOp& DragDropOp = *StaticCastSharedPtr<FAssetDragDropOp>(Operation);
			TArray<FAssetIdentifier> AssetIdentifiers;

			IAssetManagerEditorModule::ExtractAssetIdentifiersFromAssetDataList(DragDropOp.GetAssets(), AssetIdentifiers);
			SetRootAssetIdentifiers(AssetIdentifiers);
			bWasDropHandled = true;
		}
	}

	return bWasDropHandled ? FReply::Handled() : FReply::Unhandled();
}

void SSizeMap::SetRootAssetIdentifiers(const TArray<FAssetIdentifier>& NewRootAssetIdentifiers)
{
	bMemorySizeCached = false;
	RootAssetIdentifiers.Reset();
	RootAssetIdentifiers.Append(NewRootAssetIdentifiers);
	
	RefreshMap();
}

void SSizeMap::SetCurrentRegistrySource(const FAssetManagerEditorRegistrySource* RegistrySource)
{
	if (CurrentRegistrySource != RegistrySource)
	{
		CurrentRegistrySource = RegistrySource;
		bMemorySizeCached = false;
		if (!IsSizeTypeEnabled())
		{
			// If size type is disabled reset to disk size
			GetMutableDefault<USizeMapSettings>()->SizeType = IAssetManagerEditorModule::DiskSizeName;
			SizeTypeComboBoxWidget->ClearSelection();
		}
	}

	// Refresh the platform combo list to reflect current available registry sources
	TArray<const FAssetManagerEditorRegistrySource*> AvailableSources;
	EditorModule->GetAvailableRegistrySources(AvailableSources);
	SourceComboList.Reset();
	for (const FAssetManagerEditorRegistrySource* ValidSource : AvailableSources)
	{
		SourceComboList.Add(MakeShared<FString>(ValidSource->SourceName));
	}
	if (SourceComboBoxWidget.IsValid())
	{
		SourceComboBoxWidget->RefreshOptions();
	}

	RefreshMap();
}

namespace SizeMapInternals
{
	/** Serialization archive that discovers assets referenced by a specific Unreal object */
	class FAssetReferenceFinder : public FArchiveUObject
	{
	public:
		FAssetReferenceFinder(UObject* Object)
		{
			ArIsObjectReferenceCollector = true;
			ArIgnoreOuterRef = true;

			check(Object != nullptr);
			AllVisitedObjects.Add(Object);
			Object->Serialize(*this);
		}

		FArchive& operator<<(UObject*& Object)
		{
			// Only look at objects which are valid
			const bool bIsValidObject =
				IsValid(Object) &&	// Object should be valid
				!Object->HasAnyFlags(RF_Transient);	// Should not be transient
			if (bIsValidObject)
			{
				// Skip objects that we've already processed
				if (!AllVisitedObjects.Contains(Object))
				{
					AllVisitedObjects.Add(Object);

					const bool bIsAsset =
						Object->GetOuter() != nullptr &&						// Not a package itself (such as a script package like '/Script/Engine')
						Object->GetOuter()->IsA(UPackage::StaticClass()) &&	// Only want outer assets (these should be the only public assets, anyway)
						Object->HasAllFlags(RF_Public);						// Assets should be public

					if (bIsAsset)
					{
						ReferencedAssets.Add(Object);
					}
					else
					{
						// It's probably an inner object.  Recursively serialize.
						Object->Serialize(*this);

						// Make sure the object's class is serialized too, so that we catch any assets referenced from the class defaults
						AllVisitedObjects.Add(Object->GetClass());
						Object->GetClass()->Serialize(*this);	// @todo sizemap urgent: Doesn't really seem to be needed
					}
				}
			}
			return *this;
		}

		TSet< UObject* >& GetReferencedAssets()
		{
			return ReferencedAssets;
		}

	protected:
		/** The set of referenced assets */
		TSet< UObject* > ReferencedAssets;

		/** Set of all objects we've visited, so we don't follow cycles */
		TSet< UObject* > AllVisitedObjects;
	};

	/** Given a size in bytes and a boolean that indicates whether the size is actually known to be correct, returns a pretty
		string to represent that size, such as "256.0 MB", or "unknown size" */
	static FString MakeBestSizeString(const SIZE_T SizeInBytes, const bool bHasKnownSize)
	{
		FText SizeText;

		if (SizeInBytes < 1000)
		{
			// We ended up with bytes, so show a decimal number
			SizeText = FText::AsMemory(SizeInBytes);
		}
		else
		{
			// Show a fractional number with the best possible units
			FNumberFormattingOptions NumberFormattingOptions;
			NumberFormattingOptions.MaximumFractionalDigits = 1;	// @todo sizemap: We could make the number of digits customizable in the UI
			NumberFormattingOptions.MinimumFractionalDigits = 0;
			NumberFormattingOptions.MinimumIntegralDigits = 1;

			SizeText = FText::AsMemory(SizeInBytes, &NumberFormattingOptions);
		}

		if (!bHasKnownSize)
		{
			if (SizeInBytes == 0)
			{
				SizeText = LOCTEXT("UnknownSize", "unknown size");
			}
			else
			{
				SizeText = FText::Format(LOCTEXT("UnknownSizeButAtLeastThisBigFmt", "at least {0}"), SizeText);
			}
		}

		return SizeText.ToString();
	}
}

void SSizeMap::GatherDependenciesRecursively(TSharedPtr<FAssetThumbnailPool>& InAssetThumbnailPool, TMap<FAssetIdentifier, TSharedPtr<FTreeMapNodeData>>& VisitedAssetIdentifiers, const TSet<FAssetIdentifier>& AssetIdentifiers, const FPrimaryAssetId& FilterPrimaryAsset, const TSharedPtr<FTreeMapNodeData>& Node, TSharedPtr<FTreeMapNodeData>& SharedRootNode, int32& NumAssetsWhichFailedToLoad)
{
	if (!CurrentRegistrySource->HasRegistry())
	{
		return;
	}
	for (const FAssetIdentifier& AssetIdentifier : AssetIdentifiers)
	{
		FName AssetPackageName = AssetIdentifier.IsPackage() ? AssetIdentifier.PackageName : NAME_None;
		FString AssetPackageNameString = (AssetPackageName != NAME_None) ? AssetPackageName.ToString() : FString();
		FPrimaryAssetId AssetPrimaryId = AssetIdentifier.GetPrimaryAssetId();
		int32 ChunkId = UAssetManager::ExtractChunkIdFromPrimaryAssetId(AssetPrimaryId);
		int32 FilterChunkId = UAssetManager::ExtractChunkIdFromPrimaryAssetId(FilterPrimaryAsset);
		const FAssetManagerChunkInfo* FilterChunkInfo = FilterChunkId != INDEX_NONE ? CurrentRegistrySource->ChunkAssignments.Find(FilterChunkId) : nullptr;

		// Only support packages and primary assets
		if (AssetPackageName == NAME_None && !AssetPrimaryId.IsValid())
		{
			continue;
		}

		// Don't bother showing code references
		if (AssetPackageNameString.StartsWith(TEXT("/Script/")))
		{
			continue;
		}

		// Have we already added this asset to the tree?  If so, we'll either move it to a "shared" group or (if it's referenced again by the same
		// root-level asset) ignore it
		if (VisitedAssetIdentifiers.Contains(AssetIdentifier))
		{
			// OK, we've determined that this asset has already been referenced by something else in our tree.  We'll move it to a "shared" group
			// so all of the assets that are referenced in multiple places can be seen together.
			TSharedPtr<FTreeMapNodeData> ExistingNode = VisitedAssetIdentifiers[AssetIdentifier];

			// Is the existing node not already under the "shared" group?  Note that it might still be (indirectly) under
			// the "shared" group, in which case we'll still want to move it up to the root since we've figured out that it is
			// actually shared between multiple assets which themselves may be shared
			if (ExistingNode->Parent != SharedRootNode.Get())
			{
				// Don't bother moving any of the assets at the root level into a "shared" bucket.  We're only trying to best
				// represent the memory used when all of the root-level assets have become loaded.  It's OK if root-level assets
				// are referenced by other assets in the set -- we don't need to indicate they are shared explicitly
				FTreeMapNodeData* ExistingNodeParent = ExistingNode->Parent;
				check(ExistingNodeParent != nullptr);
				const bool bExistingNodeIsAtRootLevel = ExistingNodeParent->Parent == nullptr || RootAssetIdentifiers.Contains(AssetIdentifier);
				if (!bExistingNodeIsAtRootLevel)
				{
					// OK, the current asset (AssetIdentifier) is definitely not a root level asset, but its already in the tree
					// somewhere as a non-shared, non-root level asset.  We need to make sure that this Node's reference is not from the
					// same root-level asset as the ExistingNodeInTree.  Otherwise, there's no need to move it to a 'shared' group.
					FTreeMapNodeData* MyParentNode = Node.Get();
					check(MyParentNode != nullptr);
					FTreeMapNodeData* MyRootLevelAssetNode = MyParentNode;
					while (MyRootLevelAssetNode->Parent != nullptr && MyRootLevelAssetNode->Parent->Parent != nullptr)
					{
						MyRootLevelAssetNode = MyRootLevelAssetNode->Parent;
					}
					if (MyRootLevelAssetNode->Parent == nullptr)
					{
						// No root asset (Node must be a root level asset itself!)
						MyRootLevelAssetNode = nullptr;
					}

					// Find the existing node's root level asset node
					FTreeMapNodeData* ExistingNodeRootLevelAssetNode = ExistingNodeParent;
					while (ExistingNodeRootLevelAssetNode->Parent->Parent != nullptr)
					{
						ExistingNodeRootLevelAssetNode = ExistingNodeRootLevelAssetNode->Parent;
					}

					// If we're being referenced by another node within the same asset, no need to move it to a 'shared' group.  
					if (MyRootLevelAssetNode != ExistingNodeRootLevelAssetNode)
					{
						// This asset was already referenced by something else (or was in our top level list of assets to display sizes for)
						if (!SharedRootNode.IsValid())
						{
							// Find the root-most tree node
							FTreeMapNodeData* RootNode = MyParentNode;
							while (RootNode->Parent != nullptr)
							{
								RootNode = RootNode->Parent;
							}

							SharedRootNode = MakeShareable(new FTreeMapNodeData());
							RootNode->Children.Add(SharedRootNode);
							SharedRootNode->Parent = RootNode;	// Keep back-pointer to parent node
						}

						// Reparent the node that we've now determined to be shared
						ExistingNode->Parent->Children.Remove(ExistingNode);
						SharedRootNode->Children.Add(ExistingNode);
						ExistingNode->Parent = SharedRootNode.Get();
					}
				}
			}
		}
		else
		{
			// This asset is new to us so far!  Let's add it to the tree.  Later as we descend through references, we might find that the
			// asset is referenced by something else as well, in which case we'll pull it out and move it to a "shared" top-level box
			FTreeMapNodeDataRef ChildTreeMapNode = MakeShareable(new FTreeMapNodeData());
			Node->Children.Add(ChildTreeMapNode);
			ChildTreeMapNode->Parent = Node.Get();	// Keep back-pointer to parent node

			VisitedAssetIdentifiers.Add(AssetIdentifier, ChildTreeMapNode);

			FNodeSizeMapData& NodeSizeMapData = NodeSizeMapDataMap.Add(ChildTreeMapNode);

			// Set some defaults for this node.  These will be used if we can't actually locate the asset.
			if (AssetPackageName != NAME_None)
			{
				NodeSizeMapData.AssetData.AssetName = AssetPackageName;
				NodeSizeMapData.AssetData.AssetClassPath = FTopLevelAssetPath(TEXT("/None"), *LOCTEXT("MissingAsset", "MISSING!").ToString());

				FAssetData FoundData = CurrentRegistrySource->GetAssetByPackageName(AssetPackageName);
				if (FoundData.IsValid())
				{
					NodeSizeMapData.AssetData = MoveTemp(FoundData);
				}
			}
			else
			{
				NodeSizeMapData.AssetData = IAssetManagerEditorModule::CreateFakeAssetDataFromPrimaryAssetId(AssetPrimaryId);
			}
			
			NodeSizeMapData.AssetSize = 0;
			NodeSizeMapData.bHasKnownSize = false;
			NodeSizeMapData.ExternalPackageCount = 0;

			if (NodeSizeMapData.AssetData.IsValid())
			{
				FAssetManagerDependencyQuery DependencyQuery = FAssetManagerDependencyQuery::None();
				if (AssetPackageName != NAME_None)
				{
					DependencyQuery.Categories = UE::AssetRegistry::EDependencyCategory::Package;
					DependencyQuery.Flags = UE::AssetRegistry::EDependencyQuery::Hard;
				}
				else
				{
					DependencyQuery.Categories = UE::AssetRegistry::EDependencyCategory::Manage;
					DependencyQuery.Flags = UE::AssetRegistry::EDependencyQuery::Direct;
				}
				
				if (GetDefault<USizeMapSettings>()->DependencyType == ESizeMapDependencyType::EditorOnly)
				{
					DependencyQuery.Flags |= UE::AssetRegistry::EDependencyQuery::EditorOnly;
				}
				else if (GetDefault<USizeMapSettings>()->DependencyType == ESizeMapDependencyType::Game)
				{
					DependencyQuery.Flags |= UE::AssetRegistry::EDependencyQuery::Game;
				}

				TArray<FAssetIdentifier> References;
				
				if (ChunkId != INDEX_NONE)
				{
					// Look in the platform state
					const FAssetManagerChunkInfo* FoundChunkInfo = CurrentRegistrySource->ChunkAssignments.Find(ChunkId);
					if (FoundChunkInfo)
					{
						References.Append(FoundChunkInfo->ExplicitAssets.Array());
					}
				}
				else
				{
					CurrentRegistrySource->GetDependencies(AssetIdentifier, References, DependencyQuery.Categories, DependencyQuery.Flags);
				}
				
				// Filter for registry source
				IAssetManagerEditorModule::Get().FilterAssetIdentifiersForCurrentRegistrySource(References, DependencyQuery, true);

				TSet<FAssetIdentifier> ReferencedAssetIdentifiers;

				for (FAssetIdentifier& FoundAssetIdentifier : References)
				{
					if (FoundAssetIdentifier.IsPackage())
					{
						FName FoundPackageName = FoundAssetIdentifier.PackageName;

						if (FoundPackageName != NAME_None)
						{
							if (FilterChunkId != INDEX_NONE)
							{
								if (!FilterChunkInfo || !FilterChunkInfo->AllAssets.Contains(FoundPackageName))
								{
									// Not found in the chunk list, skip
									continue;
								}
							} 
							else if (FilterPrimaryAsset.IsValid())
							{
								// Check to see if this is managed by the filter asset
								TArray<FAssetIdentifier> Managers;
								CurrentRegistrySource->GetReferencers(FoundAssetIdentifier, Managers, UE::AssetRegistry::EDependencyCategory::Manage);

								if (!Managers.Contains(FilterPrimaryAsset))
								{
									continue;
								}
							}

							ReferencedAssetIdentifiers.Add(FoundPackageName);
						}
					}
					else
					{
						ReferencedAssetIdentifiers.Add(FoundAssetIdentifier);
					}
				}
				int64 FoundSize = 0;

				if (AssetPackageName != NAME_None)
				{
					if (EditorModule->GetIntegerValueForCustomColumn(NodeSizeMapData.AssetData, GetDefault<USizeMapSettings>()->SizeType, FoundSize, CurrentRegistrySource))
					{
						// If we're reading cooked data, this will fail for dependencies that are editor only. This is fine, they will have 0 size
						NodeSizeMapData.AssetSize = FoundSize;
						NodeSizeMapData.bHasKnownSize = true;
					}
				}
				else
				{
					// Virtual node, size is known to be 0
					NodeSizeMapData.bHasKnownSize = true;
				}

				// External / generated packages discovery
				GatherExternalPackages(NodeSizeMapData.AssetData, AssetPackageName, ChildTreeMapNode, VisitedAssetIdentifiers);

				// Now visit all of the assets that we are referencing
				GatherDependenciesRecursively(InAssetThumbnailPool, VisitedAssetIdentifiers, ReferencedAssetIdentifiers, ChunkId != INDEX_NONE ? AssetPrimaryId : FilterPrimaryAsset, ChildTreeMapNode, SharedRootNode, NumAssetsWhichFailedToLoad);
			}
			else
			{
				++NumAssetsWhichFailedToLoad;
			}
		}
	}
}

void SSizeMap::GatherExternalPackages(const FAssetData CurrentAssetData, FName AssetPackageName, TSharedRef<FTreeMapNodeData> ChildTreeMapNode, TMap<FAssetIdentifier, TSharedPtr<FTreeMapNodeData>>& VisitedAssetIdentifiers)
{
	auto GetExternalPackageSize = [this](const FAssetData& ExternalPackageAsset) -> int64
	{
		int64 Size = 0;
		if (EditorModule->GetIntegerValueForCustomColumn(ExternalPackageAsset, GetDefault<USizeMapSettings>()->SizeType, Size, CurrentRegistrySource))
		{
			return Size;
		}
		return 0;
	};

	auto AddExternalPackageNode = [&](const FAssetData& ExternalPackageAsset, const TSharedPtr<FTreeMapNodeData>& ParentNode)
	{
		FAssetIdentifier AssetIdentifier(ExternalPackageAsset.PackageName);
		if (VisitedAssetIdentifiers.Contains(AssetIdentifier))
		{
			return;
		}

		FTreeMapNodeDataRef TreeNode = MakeShareable(new FTreeMapNodeData());
		ParentNode->Children.Add(TreeNode);
		TreeNode->Parent = ParentNode.Get();

		VisitedAssetIdentifiers.Add(AssetIdentifier, TreeNode);

		FNodeSizeMapData& ExternalPackageNode = NodeSizeMapDataMap.Add(TreeNode);
		ExternalPackageNode.AssetData = ExternalPackageAsset;
		ExternalPackageNode.AssetSize = GetExternalPackageSize(ExternalPackageAsset);
		ExternalPackageNode.bHasKnownSize = true;
	};

	if (CurrentRegistrySource->bIsEditor)
	{
		// Scan external actor and object paths
		TArray<FAssetData> ExternalPackagesAssets;
		{
			FARFilter QueryFilter = FExternalObjectAndActorDependencyGatherer::GetQueryFilter(AssetPackageName);
			CurrentRegistrySource->GetAssets(QueryFilter, ExternalPackagesAssets);
		}

		// Group external packages by class
		if (ExternalPackagesAssets.Num() > 0)
		{
			TMap<FTopLevelAssetPath, TArray<const FAssetData*>> ClassGroups;
			for (const FAssetData& ExternalPackageAsset : ExternalPackagesAssets)
			{
				ClassGroups.FindOrAdd(ExternalPackageAsset.AssetClassPath).Add(&ExternalPackageAsset);
			}

			for (const TPair<FTopLevelAssetPath, TArray<const FAssetData*>>& ClassGroup : ClassGroups)
			{
				const FTopLevelAssetPath ClassPath = ClassGroup.Key;
				const TArray<const FAssetData*> Assets = ClassGroup.Value;

				int64 GroupTotalSize = 0;
				for (const FAssetData* Asset : Assets)
				{
					GroupTotalSize += GetExternalPackageSize(*Asset);
				}

				FTreeMapNodeDataRef GroupNode = MakeShareable(new FTreeMapNodeData());
				ChildTreeMapNode->Children.Add(GroupNode);
				GroupNode->Parent = &ChildTreeMapNode.Get();

				const FString ClassName = ClassPath.GetAssetName().ToString();
				GroupNode->Name = FString::Printf(TEXT("%s  (%d %s, %s)"), *ClassName, Assets.Num(), Assets.Num() == 1 ? TEXT("package") : TEXT("packages"), *SizeMapInternals::MakeBestSizeString(GroupTotalSize, true));
				GroupNode->Size = 0.0f;

				for (const FAssetData* Asset : Assets)
				{
					AddExternalPackageNode(*Asset, GroupNode);
				}
			}

			// Store count for display in FinalizeNodesRecursively
			NodeSizeMapDataMap.FindChecked(ChildTreeMapNode).ExternalPackageCount = ExternalPackagesAssets.Num();
		}
	}
	else
	{
		// Cooked: scan generated packages
		TArray<FAssetData> GeneratedAssets;
		{
			const FString GeneratedPath = AssetPackageName.ToString() / FPackageName::GetGeneratedPackageSubPath();

			FARFilter GeneratedFilter;
			GeneratedFilter.bRecursivePaths = true;
			GeneratedFilter.bIncludeOnlyOnDiskAssets = true;
			GeneratedFilter.PackagePaths.Add(FName(*GeneratedPath));

			CurrentRegistrySource->GetAssets(GeneratedFilter, GeneratedAssets);
		}

		if (GeneratedAssets.Num() > 0)
		{
			// Compute children's total size before adding nodes
			SIZE_T ChildrenTotalSize = 0;
			for (const FAssetData& GenAsset : GeneratedAssets)
			{
				ChildrenTotalSize += GetExternalPackageSize(GenAsset);
			}

			// Put all the generated packages in a single "Generated Packages" container
			FTreeMapNodeDataRef GeneratedGroupNode = MakeShareable(new FTreeMapNodeData());
			ChildTreeMapNode->Children.Add(GeneratedGroupNode);
			GeneratedGroupNode->Parent = &ChildTreeMapNode.Get();
			GeneratedGroupNode->Name = FString::Printf(TEXT("Generated Packages  (%d %s, %s)"), GeneratedAssets.Num(), GeneratedAssets.Num() == 1 ? TEXT("package") : TEXT("packages"), *SizeMapInternals::MakeBestSizeString(ChildrenTotalSize, true));
			GeneratedGroupNode->Size = 0.0f;

			for (const FAssetData& GenAsset : GeneratedAssets)
			{
				AddExternalPackageNode(GenAsset, GeneratedGroupNode);
			}

			// Substract the generated packages size from the generator package size
			{
				FNodeSizeMapData& GeneratorPackageEntry = NodeSizeMapDataMap.FindChecked(ChildTreeMapNode);
				GeneratorPackageEntry.ExternalPackageCount = GeneratedAssets.Num();
				GeneratorPackageEntry.AssetSize -= FMath::Min(GeneratorPackageEntry.AssetSize, ChildrenTotalSize);
			}
		}
	}
}

void SSizeMap::FinalizeNodesRecursively(TSharedPtr<FTreeMapNodeData>& Node, const TSharedPtr<FTreeMapNodeData>& SharedRootNode, int32& TotalAssetCount, SIZE_T& TotalSize, bool& bAnyUnknownSizes)
{
	// Process children first, so we can get the totals for the root node and shared nodes
	int32 SubtreeAssetCount = 0;
	SIZE_T SubtreeSize = 0;
	bool bAnyUnknownSizesInSubtree = false;
	{
		for (TSharedPtr<FTreeMapNodeData> ChildNode : Node->Children)
		{
			FinalizeNodesRecursively(ChildNode, SharedRootNode, SubtreeAssetCount, SubtreeSize, bAnyUnknownSizesInSubtree);
		}

		TotalAssetCount += SubtreeAssetCount;
		TotalSize += SubtreeSize;
		if (bAnyUnknownSizesInSubtree)
		{
			bAnyUnknownSizes = true;
		}
	}

	if (Node == SharedRootNode)
	{
		Node->Name = FString::Printf(TEXT("%s  (%s)"),
			*LOCTEXT("SharedGroupName", "*SHARED*").ToString(),
			*SizeMapInternals::MakeBestSizeString(SubtreeSize, !bAnyUnknownSizes));

		// Container nodes are always auto-sized
		Node->Size = 0.0f;
	}
	else if (Node->Parent == nullptr)
	{
		// Tree root is always auto-sized
		Node->Size = 0.0f;
	}
	else if (!NodeSizeMapDataMap.Contains(Node.ToSharedRef()))
	{
		// Container node without asset data
		Node->Size = 0.0f;
	}
	else
	{
		// Make a copy as the map may get resized
		FNodeSizeMapData NodeSizeMapData = NodeSizeMapDataMap.FindChecked(Node.ToSharedRef());
		FPrimaryAssetId PrimaryAssetId = IAssetManagerEditorModule::ExtractPrimaryAssetIdFromFakeAssetData(NodeSizeMapData.AssetData);

		++TotalAssetCount;
		TotalSize += NodeSizeMapData.AssetSize;

		if (!NodeSizeMapData.bHasKnownSize)
		{
			bAnyUnknownSizes = true;
		}

		// Setup a thumbnail
		const FSlateBrush* DefaultThumbnailSlateBrush;
		{
			// For non-class types, use the default based upon the actual asset class
			// This has the side effect of not showing a class icon for assets that don't have a proper thumbnail image available
			bool bIsClassType = false;
			const UClass* ThumbnailClass = FClassIconFinder::GetIconClassForAssetData(NodeSizeMapData.AssetData, &bIsClassType);
			const FName DefaultThumbnail = (bIsClassType) ? NAME_None : FName(*FString::Printf(TEXT("ClassThumbnail.%s"), *NodeSizeMapData.AssetData.AssetClassPath.GetAssetName().ToString()));
			DefaultThumbnailSlateBrush = FClassIconFinder::FindThumbnailForClass(ThumbnailClass, DefaultThumbnail);

			// @todo sizemap urgent: Actually implement rendered thumbnail support, not just class-based background images

			// const int32 ThumbnailSize = 128;	// @todo sizemap: Hard-coded thumbnail size.  Move this elsewhere
			//	TSharedRef<FAssetThumbnail> AssetThumbnail( new FAssetThumbnail( NodeSizeMapData.AssetData, ThumbnailSize, ThumbnailSize, AssetThumbnailPool ) );
			//	ChildTreeMapNode->AssetThumbnail = AssetThumbnail->MakeThumbnailImage();
		}

		if (PrimaryAssetId.IsValid())
		{
			Node->LogicalName = PrimaryAssetId.ToString();
		}
		else
		{
			Node->LogicalName = NodeSizeMapData.AssetData.PackageName.ToString();
		}

		// For generated packages, use the package short name instead of the asset name
		FString DisplayName;
		{
			const FString PackageNameStr = NodeSizeMapData.AssetData.PackageName.ToString();
			if (FPackageName::IsUnderGeneratedPackageSubPath(PackageNameStr))
			{
				DisplayName = FPackageName::GetShortName(PackageNameStr);
			}
			else
			{
				DisplayName = NodeSizeMapData.AssetData.AssetName.ToString();
			}
		}

		if (Node->IsLeafNode())
		{
			Node->CenterText = SizeMapInternals::MakeBestSizeString(NodeSizeMapData.AssetSize, NodeSizeMapData.bHasKnownSize);

			Node->Size = NodeSizeMapData.AssetSize;

			// The STreeMap widget is not expecting zero-sized leaf nodes.  So we make them very small instead.
			if (Node->Size == 0)
			{
				Node->Size = 1;
			}

			// Leaf nodes get a background picture
			Node->BackgroundBrush = DefaultThumbnailSlateBrush;

			if (PrimaryAssetId.IsValid())
			{
				// "Asset name"
				// "Asset type"
				Node->Name = PrimaryAssetId.ToString();
			}
			else
			{
				// "Asset name"
				// "Asset type"
				Node->Name = DisplayName;
				Node->Name2 = NodeSizeMapData.AssetData.AssetClassPath.ToString();
			}
		}
		else
		{
			// Container nodes are always auto-sized
			Node->Size = 0.0f;

			const FString SizeString = SizeMapInternals::MakeBestSizeString(SubtreeSize + NodeSizeMapData.AssetSize, !bAnyUnknownSizesInSubtree && NodeSizeMapData.bHasKnownSize);

			// Append external/generated package count when present
			FString ExternalPackageSuffix;
			if (NodeSizeMapData.ExternalPackageCount > 0)
			{
				const bool bIsCooked = CurrentRegistrySource && !CurrentRegistrySource->bIsEditor;
				ExternalPackageSuffix = FString::Printf(TEXT(", %d %s %s"), NodeSizeMapData.ExternalPackageCount, bIsCooked ? TEXT("generated") : TEXT("external"), NodeSizeMapData.ExternalPackageCount == 1 ? TEXT("package") : TEXT("packages"));
			}

			if (PrimaryAssetId.IsValid())
			{
				Node->Name = FString::Printf(TEXT("%s  (%s%s)"),
					*PrimaryAssetId.ToString(),
					*SizeString,
					*ExternalPackageSuffix);
			}
			else
			{
				// "Asset name (asset type, size, N external/Generated Package(s))"
				Node->Name = FString::Printf(TEXT("%s  (%s, %s%s)"),
					*DisplayName,
					*NodeSizeMapData.AssetData.AssetClassPath.ToString(),
					*SizeString,
					*ExternalPackageSuffix);
			}

			const bool bNeedsSelfNode = NodeSizeMapData.AssetSize > 0;
			if (bNeedsSelfNode)
			{
				// We have children, so make some space for our own asset's size within our box
				FTreeMapNodeDataRef ChildSelfTreeMapNode = MakeShareable(new FTreeMapNodeData());
				Node->Children.Add(ChildSelfTreeMapNode);
				ChildSelfTreeMapNode->Parent = Node.Get();	// Keep back-pointer to parent node

				// Map the "self" node to the same node data as its parent
				NodeSizeMapDataMap.Add(ChildSelfTreeMapNode, NodeSizeMapData);

				// "*SELF*"
				// "Asset type"
				ChildSelfTreeMapNode->Name = LOCTEXT("SelfNodeLabel", "*SELF*").ToString();
				ChildSelfTreeMapNode->Name2 = NodeSizeMapData.AssetData.AssetClassPath.ToString();

				ChildSelfTreeMapNode->CenterText = SizeMapInternals::MakeBestSizeString(NodeSizeMapData.AssetSize, NodeSizeMapData.bHasKnownSize);
				ChildSelfTreeMapNode->Size = NodeSizeMapData.AssetSize;

				// Leaf nodes get a background picture
				ChildSelfTreeMapNode->BackgroundBrush = DefaultThumbnailSlateBrush;
			}
		}

	}

	// Sort all of my child nodes alphabetically.  This is just so that we get deterministic results when viewing the
	// same sets of assets.
	Node->Children.StableSort(
		[](const FTreeMapNodeDataPtr& A, const FTreeMapNodeDataPtr& B)
	{
		return A->Name < B->Name;
	}
	);
}


void SSizeMap::RefreshMap()
{
	if (AssetRegistry->IsLoadingAssets())
	{
		// We are still discovering assets, listen for the completion delegate before building the graph
		if (!AssetRegistry->OnFilesLoaded().IsBoundToObject(this))
		{
			AssetRegistry->OnFilesLoaded().AddSP(this, &SSizeMap::OnInitialAssetRegistrySearchComplete);
		}
		return;
	}

	bool bShowSlowTask = false;
	if (GetDefault<USizeMapSettings>()->SizeType == IAssetManagerEditorModule::ResourceSizeName)
	{
		if (!bMemorySizeCached)
		{
			bMemorySizeCached = true;
			bShowSlowTask = true;
		}
	}

	// If we're refreshing memory start a slow task as we need to load assets
	FScopedSlowTask SlowTask(0, LOCTEXT("ComputeMemorySizeSlowTask", "Finding Memory Size..."), bShowSlowTask);
	SlowTask.MakeDialog();

	// Wipe the current tree out
	RootTreeMapNode->Children.Empty();
	NodeSizeMapDataMap.Empty();

	// First, do a pass to gather asset dependencies and build up a tree
	TMap<FAssetIdentifier, TSharedPtr<FTreeMapNodeData>> VisitedAssetIdentifiers;
	TSharedPtr<FTreeMapNodeData> SharedRootNode;
	int32 NumAssetsWhichFailedToLoad = 0;
	GatherDependenciesRecursively(AssetThumbnailPool, VisitedAssetIdentifiers, RootAssetIdentifiers, FPrimaryAssetId(), RootTreeMapNode, SharedRootNode, NumAssetsWhichFailedToLoad);

	// Next, do another pass over our tree to and count how big the assets are and to set the node labels.  Also in this pass, we may
	// create some additional "self" nodes for assets that have children but also take up size themselves.
	int32 TotalAssetCount = 0;
	SIZE_T TotalSize = 0;
	bool bAnyUnknownSizes = false;
	FinalizeNodesRecursively(RootTreeMapNode, SharedRootNode, TotalAssetCount, TotalSize, bAnyUnknownSizes);

	// Create a nice name for the tree!
	if (RootAssetIdentifiers.Num() == 1 && !SharedRootNode.IsValid())
	{
		FString OnlyAssetName = RootAssetIdentifiers.FindArbitraryElement()->ToString();
		if (OnlyAssetName.StartsWith(TEXT("/")))
		{
			OnlyAssetName = FPackageName::GetShortName(*OnlyAssetName);
		}

		if (RootTreeMapNode->Children.Num() > 0)
		{
			// The root will only have one child, so go ahead and use that child as the actual root
			FTreeMapNodeDataPtr OnlyChild = RootTreeMapNode->Children[0];
			OnlyChild->CopyNodeInto(*RootTreeMapNode);
			RootTreeMapNode->Children = OnlyChild->Children;
			RootTreeMapNode->Parent = nullptr;
			for (const auto& ChildNode : RootTreeMapNode->Children)
			{
				ChildNode->Parent = RootTreeMapNode.Get();
			}

			NodeSizeMapDataMap.Add(RootTreeMapNode.ToSharedRef(), NodeSizeMapDataMap.FindRef(OnlyChild.ToSharedRef()));
		}

		// Set the overview text
		OverviewText = FText::Format(LOCTEXT("RootNode_Format", "Size map for {0}  ({1} total {1}|plural(one=asset,other=assets)))"),
			FText::AsCultureInvariant(OnlyAssetName),
			TotalAssetCount);
	}
	else
	{
		FString AssetNames;
		FString LogicalName;

		for (const FAssetIdentifier& Identifier : RootAssetIdentifiers)
		{
			FString IdentifierString = Identifier.ToString();

			if (Identifier.IsPackage())
			{
				IdentifierString = FPackageName::GetShortName(*IdentifierString);
			}

			if (AssetNames.Len() != 0)
			{
				AssetNames += TEXT(", ");
			}
			else
			{
				LogicalName = IdentifierString + LOCTEXT("AndOthers", "AndOthers").ToString();
			}
			AssetNames += IdentifierString;
		}

		// Multiple assets (or at least some shared assets) at the root level
		RootTreeMapNode->BackgroundBrush = nullptr;
		RootTreeMapNode->Size = 0.0f;
		RootTreeMapNode->Parent = nullptr;
		RootTreeMapNode->Name = FString::Printf(TEXT("%s  (%s)"),
			*AssetNames.Left(96),
			*SizeMapInternals::MakeBestSizeString(TotalSize, !bAnyUnknownSizes));
		RootTreeMapNode->LogicalName = LogicalName;

		OverviewText = FText::Format(LOCTEXT("RootNode_FormatForAssets", "Size map for {0} {0}|plural(one=asset,other=assets)  ({1} total {1}|plural(one=asset,other=assets)))"),
			RootAssetIdentifiers.Num(),
			TotalAssetCount);
	}

	// OK, now refresh the actual tree map widget so our new tree will be displayed.
	const bool bShouldPlayTransition = false;
	TreeMapWidget->RebuildTreeMap(bShouldPlayTransition);
}

void SSizeMap::OnInitialAssetRegistrySearchComplete()
{
	bMemorySizeCached = false;
	RefreshMap();
}

TSharedPtr<FTreeMapNodeData> SSizeMap::GetCurrentTreeNode(bool bConsumeSelection) const
{
	TSharedPtr<FTreeMapNodeData> ReturnTreeNode = nullptr;
	if (CurrentSelectedNode.IsValid())
	{
		ReturnTreeNode = CurrentSelectedNode.Pin();
		if (bConsumeSelection)
		{
			CurrentSelectedNode.Reset();
		}
	}
	if (!ReturnTreeNode.IsValid())
	{
		ReturnTreeNode = TreeMapWidget->GetTreeRoot();
	}
	return ReturnTreeNode;
}

const SSizeMap::FNodeSizeMapData* SSizeMap::GetCurrentSizeMapData(bool bConsumeSelection) const
{
	TSharedPtr<FTreeMapNodeData> ReturnTreeNode = GetCurrentTreeNode(bConsumeSelection);
	const FNodeSizeMapData* NodeSizeMapData = nullptr;

	if (ReturnTreeNode.IsValid())
	{
		NodeSizeMapData = NodeSizeMapDataMap.Find(ReturnTreeNode.ToSharedRef());
	}
	return NodeSizeMapData;
}

bool SSizeMap::IsAnythingSelected() const
{
	const FNodeSizeMapData* NodeSizeMapData = GetCurrentSizeMapData(false);

	if (NodeSizeMapData && NodeSizeMapData->AssetData.IsValid())
	{
		return true;
	}
	return false;
}

void SSizeMap::FindInContentBrowser() const
{
	const FNodeSizeMapData* NodeSizeMapData = GetCurrentSizeMapData(true);
	if (NodeSizeMapData)
	{
		TArray<FAssetData> Assets;
		Assets.Add(NodeSizeMapData->AssetData);

		GEditor->SyncBrowserToObjects(Assets);
	}
}

void SSizeMap::EditSelectedAssets() const
{
	const FNodeSizeMapData* NodeSizeMapData = GetCurrentSizeMapData(true);
	if (NodeSizeMapData)
	{
		TArray<FSoftObjectPath> AssetPaths;
		AssetPaths.Add(NodeSizeMapData->AssetData.GetSoftObjectPath());

		GEditor->GetEditorSubsystem<UAssetEditorSubsystem>()->OpenEditorsForAssets(AssetPaths);
	}
}
	
void SSizeMap::FindReferencesForSelectedAssets() const
{
	const FNodeSizeMapData* NodeSizeMapData = GetCurrentSizeMapData(true);
	if (NodeSizeMapData)
	{
		TArray<FAssetIdentifier> AssetIdentifiers;
		TArray<FAssetData> AssetDataList;
		
		AssetDataList.Add(NodeSizeMapData->AssetData);
		
		IAssetManagerEditorModule::ExtractAssetIdentifiersFromAssetDataList(AssetDataList, AssetIdentifiers);
		IAssetManagerEditorModule::Get().OpenReferenceViewerUI(AssetIdentifiers);
	}
}

void SSizeMap::ShowAssetAuditForSelectedAssets() const
{
	const FNodeSizeMapData* NodeSizeMapData = GetCurrentSizeMapData(true);
	if (NodeSizeMapData)
	{
		TArray<FName> PackageNames;
		PackageNames.Add(NodeSizeMapData->AssetData.PackageName);

		IAssetManagerEditorModule::Get().OpenAssetAuditUI(PackageNames);
	}
}

void SSizeMap::ShowAssetAuditForReferencedAssets() const
{
	TSharedPtr<FTreeMapNodeData> ReturnTreeNode = GetCurrentTreeNode(true);

	if (ReturnTreeNode.IsValid())
	{
		TSet<FName> PackageNameSet;
		GetReferencedPackages(PackageNameSet, ReturnTreeNode, true);

		IAssetManagerEditorModule::Get().OpenAssetAuditUI(PackageNameSet.Array());
	}
}

void SSizeMap::CopyPackageName() const
{
	const FNodeSizeMapData* NodeSizeMapData = GetCurrentSizeMapData(true);
	if (NodeSizeMapData)
	{
		FPlatformApplicationMisc::ClipboardCopy(*NodeSizeMapData->AssetData.PackageName.ToString());
	}
}

void SSizeMap::GetReferencedPackages(TSet<FName>& OutPackageNames, TSharedPtr<FTreeMapNodeData> RootNode, bool bRecurse) const
{
	if (!RootNode.IsValid())
	{
		return;
	}

	// Recursively iterate children
	TArray<TSharedPtr<FTreeMapNodeData>> NodesToIterate = RootNode->Children;

	while (NodesToIterate.Num() > 0)
	{
		// Pop off the end, order doesn't matter here
		TSharedPtr<FTreeMapNodeData> CurrentNode = NodesToIterate.Pop(EAllowShrinking::No);

		if (CurrentNode.IsValid())
		{
			const FNodeSizeMapData* NodeSizeMapData = NodeSizeMapDataMap.Find(CurrentNode.ToSharedRef());

			if (NodeSizeMapData && !IAssetManagerEditorModule::ExtractPrimaryAssetIdFromFakeAssetData(NodeSizeMapData->AssetData).IsValid())
			{
				// If this is a real package, add it to list
				OutPackageNames.Add(NodeSizeMapData->AssetData.PackageName);
			}

			if (bRecurse)
			{
				NodesToIterate.Append(CurrentNode->Children);
			}
		}
	}
}

bool SSizeMap::CanMakeCollectionWithDependencies(ECollectionShareType::Type ShareType) const
{
	return CanMakeCollectionWithDependencies(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(), ShareType);
}

void SSizeMap::MakeCollectionWithDependencies(ECollectionShareType::Type ShareType)
{
	MakeCollectionWithDependencies(FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer(), ShareType);
}

bool SSizeMap::CanMakeCollectionWithDependencies(TSharedPtr<ICollectionContainer> CollectionContainer, ECollectionShareType::Type ShareType) const
{
	if (!ensure(CollectionContainer))
	{
		return false;
	}

	return !CollectionContainer->IsReadOnly(ShareType) && GetCurrentTreeNode(false).IsValid();
}

void SSizeMap::MakeCollectionWithDependencies(TSharedPtr<ICollectionContainer> CollectionContainer, ECollectionShareType::Type ShareType)
{
	if (!ensure(CollectionContainer))
	{
		return;
	}

	TSharedPtr<FTreeMapNodeData> ReturnTreeNode = GetCurrentTreeNode(true);

	if (ReturnTreeNode.IsValid())
	{
		TSet<FName> PackageNameSet;
		GetReferencedPackages(PackageNameSet, ReturnTreeNode, true);

		FString ShortName = ReturnTreeNode->LogicalName.Replace(TEXT(":"), TEXT("_"));
		if (ShortName.StartsWith(TEXT("/")))
		{
			ShortName = FPackageName::GetShortName(*ShortName);
		}

		FText CollectionNameAsText = FText::Format(LOCTEXT("DependenciesForAssetNames", "{0}_SizeMapDependencies"), FText::AsCultureInvariant(ShortName));

		FName CollectionName;
		CollectionContainer->CreateUniqueCollectionName(*CollectionNameAsText.ToString(), ShareType, CollectionName);

		IAssetManagerEditorModule::Get().WriteCollection(*CollectionContainer, CollectionName, ShareType, PackageNameSet.Array(), true);
	}
}

void SSizeMap::OnTreeMapNodeRightClicked(FTreeMapNodeData& TreeMapNodeData, const FPointerEvent& MouseEvent)
{
	CurrentSelectedNode = TreeMapNodeData.AsShared();

	// Determine the node type to show appropriate context menu entries
	const FNodeSizeMapData* NodeData = NodeSizeMapDataMap.Find(TreeMapNodeData.AsShared());
	const bool bIsContainerNode = (NodeData == nullptr);
	bool bIsExternalPackage = false;
	bool bIsGeneratedPackage = false;
	if (NodeData)
	{
		const FString PackageNameStr = NodeData->AssetData.PackageName.ToString();
		bIsGeneratedPackage = FPackageName::IsUnderGeneratedPackageSubPath(PackageNameStr);
		bIsExternalPackage = !bIsGeneratedPackage &&
			(PackageNameStr.Contains(FPackagePath::GetExternalActorsFolderName()) || 
			 PackageNameStr.Contains(FPackagePath::GetExternalObjectsFolderName()));
	}
	const bool bIsRegularAsset = NodeData && !bIsExternalPackage && !bIsGeneratedPackage;

	// Construct menu, this can't support keybinds
	FMenuBuilder MenuBuilder(/*bInShouldCloseWindowAfterMenuSelection=*/ true, Commands);

	MenuBuilder.BeginSection(TEXT("Asset"), LOCTEXT("AssetSectionLabel", "Asset"));
	{
		if (bIsRegularAsset)
		{
			MenuBuilder.AddMenuEntry(FGlobalEditorCommonCommands::Get().FindInContentBrowser);
			MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().OpenSelectedInAssetEditor);
		}

		if (!bIsContainerNode)
		{
			MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().CopyPaths);
		}
	}
	MenuBuilder.EndSection();

	MenuBuilder.BeginSection(TEXT("References"), LOCTEXT("ReferencesSectionLabel", "References"));
	{
		if (bIsRegularAsset || bIsExternalPackage)
		{
			MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().ViewReferences);
			MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().ViewAssetAudit);
		}

		if (!bIsGeneratedPackage)
		{
			MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().AuditReferencedObjects);
		}

		if (bIsRegularAsset)
		{
			ICollectionManager& CollectionManager = FCollectionManagerModule::GetModule().Get();

			TArray<TSharedPtr<ICollectionContainer>> CollectionContainers;
			CollectionManager.GetVisibleCollectionContainers(CollectionContainers);

			if (!CollectionContainers.IsEmpty())
			{
				FNewMenuDelegate SubMenu;
				if (CollectionContainers.Num() == 1)
				{
					if (CollectionContainers[0] == CollectionManager.GetProjectCollectionContainer())
					{
						// Use the command version to show key bindings.
						SubMenu = FNewMenuDelegate::CreateSP(this, &SSizeMap::GetMakeCollectionWithDependenciesShareTypeSubMenu);
					}
					else
					{
						SubMenu = FNewMenuDelegate::CreateSP(this, &SSizeMap::GetMakeCollectionWithDependenciesShareTypeSubMenu, CollectionContainers[0]);
					}
				}
				else
				{
					// Create a sub menu to select the collection container.
					SubMenu = FNewMenuDelegate::CreateSP(this, &SSizeMap::GetMakeCollectionWithDependenciesCollectionContainersSubMenu, MoveTemp(CollectionContainers));
				}
				MenuBuilder.AddSubMenu(
					LOCTEXT("MakeCollectionWithTitle", "Make Collection With References"),
					LOCTEXT("MakeCollectionWithTooltip", "Makes a collection with all assets referenced by this node."),
					MoveTemp(SubMenu)
				);
			}
		}
	}
	MenuBuilder.EndSection();

	// Spawn context menu
	FWidgetPath WidgetPath = MouseEvent.GetEventPath() != nullptr ? *MouseEvent.GetEventPath() : FWidgetPath();
	FSlateApplication::Get().PushMenu(AsShared(), WidgetPath, MenuBuilder.MakeWidget(), MouseEvent.GetScreenSpacePosition(), FPopupTransitionEffect(FPopupTransitionEffect::ContextMenu));
}

void SSizeMap::GetMakeCollectionWithDependenciesCollectionContainersSubMenu(FMenuBuilder& MenuBuilder, TArray<TSharedPtr<ICollectionContainer>> CollectionContainers)
{
	const TSharedRef<ICollectionContainer>& ProjectCollectionContainer = FCollectionManagerModule::GetModule().Get().GetProjectCollectionContainer();

	MenuBuilder.BeginSection(NAME_None, LOCTEXT("MakeCollectionWithCollectionContainersMenuHeading", "Collection Containers"));

	for (const TSharedPtr<ICollectionContainer>& CollectionContainer : CollectionContainers)
	{
		FNewMenuDelegate SubMenu;
		if (CollectionContainer == ProjectCollectionContainer)
		{
			// Use the command version to show key bindings.
			SubMenu = FNewMenuDelegate::CreateSP(this, &SSizeMap::GetMakeCollectionWithDependenciesShareTypeSubMenu);
		}
		else
		{
			SubMenu = FNewMenuDelegate::CreateSP(this, &SSizeMap::GetMakeCollectionWithDependenciesShareTypeSubMenu, CollectionContainer);
		}
		MenuBuilder.AddSubMenu(
			CollectionContainer->GetCollectionSource()->GetTitle(),
			TAttribute<FText>(),
			MoveTemp(SubMenu));
	}

	MenuBuilder.EndSection();
}

void SSizeMap::GetMakeCollectionWithDependenciesShareTypeSubMenu(FMenuBuilder& MenuBuilder)
{
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().MakeLocalCollectionWithDependencies,
		NAME_None,
		ECollectionShareType::ToText(ECollectionShareType::CST_Local),
		ECollectionShareType::GetDescription(ECollectionShareType::CST_Local),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Local))
	);
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().MakePrivateCollectionWithDependencies,
		NAME_None,
		ECollectionShareType::ToText(ECollectionShareType::CST_Private),
		ECollectionShareType::GetDescription(ECollectionShareType::CST_Private),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Private))
	);
	MenuBuilder.AddMenuEntry(FAssetManagerEditorCommands::Get().MakeSharedCollectionWithDependencies,
		NAME_None,
		ECollectionShareType::ToText(ECollectionShareType::CST_Shared),
		ECollectionShareType::GetDescription(ECollectionShareType::CST_Shared),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Shared))
	);
}

void SSizeMap::GetMakeCollectionWithDependenciesShareTypeSubMenu(class FMenuBuilder& MenuBuilder, TSharedPtr<ICollectionContainer> CollectionContainer)
{
	MenuBuilder.AddMenuEntry(
		ECollectionShareType::ToText(ECollectionShareType::CST_Local),
		ECollectionShareType::GetDescription(ECollectionShareType::CST_Local),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Local)),
		FUIAction(
			FExecuteAction::CreateSP(this, &SSizeMap::MakeCollectionWithDependencies, CollectionContainer, ECollectionShareType::CST_Local),
			FCanExecuteAction::CreateSP(this, &SSizeMap::CanMakeCollectionWithDependencies, CollectionContainer, ECollectionShareType::CST_Local))
	);

	MenuBuilder.AddMenuEntry(
		ECollectionShareType::ToText(ECollectionShareType::CST_Private),
		ECollectionShareType::GetDescription(ECollectionShareType::CST_Private),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Private)),
		FUIAction(
			FExecuteAction::CreateSP(this, &SSizeMap::MakeCollectionWithDependencies, CollectionContainer, ECollectionShareType::CST_Private),
			FCanExecuteAction::CreateSP(this, &SSizeMap::CanMakeCollectionWithDependencies, CollectionContainer, ECollectionShareType::CST_Private))
	);

	MenuBuilder.AddMenuEntry(
		ECollectionShareType::ToText(ECollectionShareType::CST_Shared),
		ECollectionShareType::GetDescription(ECollectionShareType::CST_Shared),
		FSlateIcon(FAppStyle::GetAppStyleSetName(), ECollectionShareType::GetIconStyleName(ECollectionShareType::CST_Shared)),
		FUIAction(
			FExecuteAction::CreateSP(this, &SSizeMap::MakeCollectionWithDependencies, CollectionContainer, ECollectionShareType::CST_Shared),
			FCanExecuteAction::CreateSP(this, &SSizeMap::CanMakeCollectionWithDependencies, CollectionContainer, ECollectionShareType::CST_Shared))
	);
}

#undef LOCTEXT_NAMESPACE
