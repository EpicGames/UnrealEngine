// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetManagerEditorModule.h"
#include "CollectionManagerTypes.h"
#include "Widgets/SCompoundWidget.h"
#include "AssetRegistry/AssetData.h"
#include "SSizeMap.generated.h"

class ICollectionContainer;
class FTreeMapNodeData;
namespace ESelectInfo { enum Type : int; }
struct FAssetIdentifier;
struct FAssetManagerEditorRegistrySource;
template <typename OptionType> class SComboBox;
class FAssetThumbnailPool;
class FUICommandList;

UENUM()
enum class ESizeMapDependencyType
{
	/** Queries all hard dependencies */
	All,
	/** Queries hard dependencies that are part of a cooked build. See UE::AssetRegistry::EDependencyQuery::Game. */
	Game,
	/** Queries hard dependencies that only exist in the editor. See UE::AssetRegistry::EDependencyQuery::EditorOnly */
	EditorOnly,
};

UCLASS(config=EditorPerProjectUserSettings)
class USizeMapSettings : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(config)
	FName SizeType = IAssetManagerEditorModule::DiskSizeName;

	UPROPERTY(config)
	ESizeMapDependencyType DependencyType = ESizeMapDependencyType::All;
};

/**
 * Tree map for displaying the size of assets
 */
class SSizeMap : public SCompoundWidget
{

public:

	SLATE_BEGIN_ARGS(SSizeMap)
	{}
	SLATE_END_ARGS()

	/** Default constructor for SSizeMap */
	SSizeMap();

	/** Destructor for SSizeMap */
	~SSizeMap();

	/**
	 * Construct the widget
	 *
	 * @param	InArgs				A declaration from which to construct the widget
	 */
	void Construct(const FArguments& InArgs);

	/** Sets the assets to view at the root of the size map.  This will rebuild the map. */
	void SetRootAssetIdentifiers(const TArray<FAssetIdentifier>& NewRootAssetIdentifiers);

	/** Called when the current registry source changes */
	void SetCurrentRegistrySource(const FAssetManagerEditorRegistrySource* RegistrySource);

protected:
	/** This struct will contain size map-specific payload data that we'll associate with tree map nodes using a hash table */
	struct FNodeSizeMapData
	{
		/** How big the asset is */
		SIZE_T AssetSize = 0;

		/** Whether it has a known size or not */
		bool bHasKnownSize = false;

		/** Number of external/generated packages discovered for this asset */
		int32 ExternalPackageCount = 0;

		/** Data from the asset registry about this asset */
		FAssetData AssetData;
	};

	/** Overrides */
	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent);
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;

	/** Called after the initial asset registry scan finishes */
	void OnInitialAssetRegistrySearchComplete();

	/** Recursively discovers and loads dependent assets, building up a tree map node hierarchy */
	void GatherDependenciesRecursively(TSharedPtr<class FAssetThumbnailPool>& AssetThumbnailPool, TMap<FAssetIdentifier, TSharedPtr<class FTreeMapNodeData>>& VisitedAssetIdentifiers, const TSet<FAssetIdentifier>& AssetIdentifiers, const FPrimaryAssetId& FilterPrimaryAsset, const TSharedPtr<FTreeMapNodeData>& ParentTreeMapNode, TSharedPtr<FTreeMapNodeData>& SharedRootNode, int32& NumAssetsWhichFailedToLoad);

	/** Gather all external/generated packages tied to the provided asset */
	void GatherExternalPackages(const FAssetData CurrentAssetData, FName AssetPackageName, TSharedRef<FTreeMapNodeData> ChildTreeMapNode, TMap<FAssetIdentifier, TSharedPtr<FTreeMapNodeData>>& VisitedAssetIdentifiers);

	/** After the node tree is built up, this function is called to generate nice labels for the nodes and to do a final clean-up pass on the tree */
	void FinalizeNodesRecursively(TSharedPtr<FTreeMapNodeData>& Node, const TSharedPtr<FTreeMapNodeData>& SharedRootNode, int32& TotalAssetCount, SIZE_T& TotalSize, bool& bAnyUnknownSizes);

	/** Refreshes the display */
	void RefreshMap();

	/** Returns the size map data associated with current tree map node. If Consume is true it unsets the current selected node */
	TSharedPtr<FTreeMapNodeData> GetCurrentTreeNode(bool bConsumeSelection = false) const;
	const FNodeSizeMapData* GetCurrentSizeMapData(bool bConsumeSelection = false) const;

	/** Returns package names that are referenced by this node, not including root node */
	void GetReferencedPackages(TSet<FName>& OutPackageNames, TSharedPtr<FTreeMapNodeData> RootNode, bool bRecurse = true) const;

	/** Called when the user right-clicks on an asset in the tree */
	void OnTreeMapNodeRightClicked(class FTreeMapNodeData& TreeMapNodeData, const FPointerEvent& MouseEvent);
	void GetMakeCollectionWithDependenciesCollectionContainersSubMenu(class FMenuBuilder& MenuBuilder, TArray<TSharedPtr<ICollectionContainer>> CollectionContainers);
	void GetMakeCollectionWithDependenciesShareTypeSubMenu(class FMenuBuilder& MenuBuilder);
	void GetMakeCollectionWithDependenciesShareTypeSubMenu(class FMenuBuilder& MenuBuilder, TSharedPtr<ICollectionContainer> CollectionContainer);

	/** Delegates for context menu */
	void FindInContentBrowser() const;
	bool IsAnythingSelected() const;
	void EditSelectedAssets() const;
	void FindReferencesForSelectedAssets() const;
	void ShowAssetAuditForSelectedAssets() const;
	void ShowAssetAuditForReferencedAssets() const;
	void CopyPackageName() const;
	bool CanMakeCollectionWithDependencies(ECollectionShareType::Type ShareType) const;
	void MakeCollectionWithDependencies(ECollectionShareType::Type ShareType);
	bool CanMakeCollectionWithDependencies(TSharedPtr<ICollectionContainer> CollectionContainer, ECollectionShareType::Type ShareType) const;
	void MakeCollectionWithDependencies(TSharedPtr<ICollectionContainer> CollectionContainer, ECollectionShareType::Type ShareType);

	/** Back button */
	FReply OnZoomOut();
	bool CanZoomOut() const;

	/** Size Type Combo Box */
	TSharedRef<SWidget> GenerateSizeTypeComboItem(TSharedPtr<FName> InItem) const;
	void HandleSizeTypeComboChanged(TSharedPtr<FName> Item, ESelectInfo::Type SelectInfo);
	FText GetSizeTypeComboText() const;
	FText GetSizeTypeText(FName SizeType) const;
	FText GetOverviewText() const;
	bool IsSizeTypeEnabled() const;

	TArray<TSharedPtr<FName>> SizeTypeComboList;
	FText OverviewText;
	bool bMemorySizeCached;
	TSharedPtr<SComboBox<TSharedPtr<FName>>> SizeTypeComboBoxWidget;

	/** Dependency Type Combo Box */
	TSharedRef<SWidget> GenerateDependencyTypeComboItem(TSharedPtr<ESizeMapDependencyType> InItem) const;
	void HandleDependencyTypeComboChanged(TSharedPtr<ESizeMapDependencyType> Item, ESelectInfo::Type SelectInfo);
	FText GetDependencyTypeComboText() const;
	FText GetDependencyTypeText(ESizeMapDependencyType DependencyType) const;

	TArray<TSharedPtr<ESizeMapDependencyType>> DependencyTypeComboList;
	TSharedPtr<SComboBox<TSharedPtr<ESizeMapDependencyType>>> DependencyTypeComboBoxWidget;

	/** Platform / Registry Source Combo Box */
	TSharedRef<SWidget> GenerateSourceComboItem(TSharedPtr<FString> InItem) const;
	void HandleSourceComboChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo);
	FText GetSourceComboText() const;

	TArray<TSharedPtr<FString>> SourceComboList;
	TSharedPtr<SComboBox<TSharedPtr<FString>>> SourceComboBoxWidget;

	/** Plugin Filter Combo Box — sets plugin assets as root identifiers */
	TSharedRef<SWidget> GeneratePluginComboItem(TSharedPtr<FString> InItem) const;
	void HandlePluginComboChanged(TSharedPtr<FString> Item, ESelectInfo::Type SelectInfo);
	FText GetPluginComboText() const;

	/** Clears root identifiers and replaces them with all assets from PluginName */
	void SetRootAssetIdentifiersFromPlugin(const FString& PluginName);

	TArray<TSharedPtr<FString>> PluginComboList;
	TSharedPtr<SComboBox<TSharedPtr<FString>>> PluginComboBoxWidget;
	FString CurrentPluginSelection; // Empty string = no plugin filter (None)

	/** Our tree map widget */
	TSharedPtr<class STreeMap> TreeMapWidget;

	/** The assets we were asked to look at */
	TSet<FAssetIdentifier> RootAssetIdentifiers;

	/** Our tree map source data */
	TSharedPtr<class FTreeMapNodeData> RootTreeMapNode;

	/** Thumbnail pool */
	TSharedPtr<class FAssetThumbnailPool> AssetThumbnailPool;

	/** Maps a tree node to our size map-specific user data for that node */
	typedef TMap<TSharedRef<FTreeMapNodeData>, FNodeSizeMapData> FNodeSizeMapDataMap;
	FNodeSizeMapDataMap NodeSizeMapDataMap;
	mutable TWeakPtr<FTreeMapNodeData> CurrentSelectedNode;

	/** The registry source to display information for */
	const FAssetManagerEditorRegistrySource* CurrentRegistrySource;

	/** Commands handled by this widget */
	TSharedPtr<FUICommandList> Commands;

	/** Cached interfaces */
	class IAssetRegistry* AssetRegistry;
	class UAssetManager* AssetManager;
	class IAssetManagerEditorModule* EditorModule;
};

