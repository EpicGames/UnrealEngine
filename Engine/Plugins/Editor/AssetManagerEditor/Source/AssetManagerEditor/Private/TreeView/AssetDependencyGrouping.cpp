// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetDependencyGrouping.h"
#include "AssetTreeNode.h"
#include "AssetTable.h"
#include "InsightsCore/Common/AsyncOperationProgress.h"
#include "InsightsCore/Common/Log.h"

#define LOCTEXT_NAMESPACE "FAssetDependencyGrouping"

INSIGHTS_IMPLEMENT_RTTI(FAssetDependencyGrouping)
INSIGHTS_IMPLEMENT_RTTI(FPluginDependencyGrouping)

////////////////////////////////////////////////////////////////////////////////////////////////////
// FAssetDependencyGrouping
////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetDependencyGrouping::FAssetDependencyGrouping()
	: FTreeNodeGrouping(
		LOCTEXT("Grouping_ByAssetDependency_ShortName", "Asset Dependency"),
		LOCTEXT("Grouping_ByAssetDependency_TitleName", "By Asset Dependency"),
		LOCTEXT("Grouping_ByAssetDependency_Desc", "Group assets based on their dependency."),
		nullptr)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetDependencyGrouping::~FAssetDependencyGrouping()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetDependencyGrouping::GroupNodes(const TArray<UE::Insights::FTableTreeNodePtr>& Nodes, UE::Insights::FTableTreeNode& ParentGroup, TWeakPtr<UE::Insights::FTable> InParentTable, UE::Insights::IAsyncOperationProgress& InAsyncOperationProgress) const
{
	using namespace UE::Insights;

	ParentGroup.ClearChildren();

	TSharedPtr<FAssetTable> AssetTable = StaticCastSharedPtr<FAssetTable>(InParentTable.Pin());
	check(AssetTable.IsValid());

	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			return;
		}

		if (NodePtr->IsGroup() || !NodePtr->Is<FAssetTreeNode>())
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		// For each (visible) asset, we will create the following hierarchy:

		// By default, the "Dependencies" node is collapsed.
		// |
		// +-- [group:{AssetName}] (self + dependencies)
		//     |
		//     +-- [group:_Self]
		//     |   |
		//     |   +-- [asset:{AssetName}]
		//     |
		//     +-- [group:Dependencies] (double click to expand)

		// When the "Dependencies" node is expanded, it will be populated with actual dependencies.
		// |
		// +-- [group:{AssetName}] (self + dependencies)
		//     |
		//     +-- [group:_Self]
		//     |   |
		//     |   +-- [asset:{AssetName}]
		//     |
		//     +-- [group:Dependencies]
		//         |
		//         +-- [asset:{DependentAsset1}]
		//         |
		//         +-- [asset:{DependentAsset2}]
		//         |
		//         +-- [group:{DependentAsset3}] (self + dependencies)
		//         |   |
		//         |   ...
		//         ...

		FAssetTreeNode& AssetNode = NodePtr->As<FAssetTreeNode>();
		const FAssetTableRow& Asset = AssetNode.GetAssetChecked();

		if (Asset.GetNumDependencies() > 0)
		{
			// Create a group for the asset node (self) + dependencies.
			FTableTreeNodePtr AssetGroupPtr = MakeShared<FAssetTreeNode>(Asset.GetNodeName(), AssetNode.GetAssetTableWeak(), AssetNode.GetRowIndex(), true);
			AssetGroupPtr->SetExpansion(false);
			ParentGroup.AddChildAndSetParent(AssetGroupPtr);

			// Add the asset node (self) under a "_self" group node.
			static FName SelfGroupName(TEXT("_Self_")); // used _ prefix to sort before "Dependencies"
			FTableTreeNodePtr SelfGroupPtr = MakeShared<FAssetTreeNode>(SelfGroupName, AssetNode.GetAssetTableWeak(), AssetNode.GetRowIndex(), true);
			SelfGroupPtr->SetExpansion(false);
			SelfGroupPtr->AddChildAndSetParent(NodePtr);
			AssetGroupPtr->AddChildAndSetParent(SelfGroupPtr);

			// Create a group node for all dependent assets of the current asset.
			// The actual nodes for the dependent assets are lazy created by this group node.
			static FName DependenciesGroupName(TEXT("Dependencies"));
			TSharedPtr<FAssetDependenciesGroupTreeNode> DependenciesGroupPtr = MakeShared<FAssetDependenciesGroupTreeNode>(DependenciesGroupName, AssetTable, AssetNode.GetRowIndex());
			DependenciesGroupPtr->SetExpansion(false);
			AssetGroupPtr->AddChildAndSetParent(DependenciesGroupPtr);
		}
		else
		{
			// No dependencies. Just add the asset node (self).
			ParentGroup.AddChildAndSetParent(NodePtr);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FPluginDependencyGrouping
////////////////////////////////////////////////////////////////////////////////////////////////////

FPluginDependencyGrouping::FPluginDependencyGrouping()
	: FTreeNodeGrouping(
		LOCTEXT("Grouping_ByPluginDependency_ShortName", "Plugin Dependency"),
		LOCTEXT("Grouping_ByPluginDependency_TitleName", "By Plugin Dependency"),
		LOCTEXT("Grouping_ByPluginDependency_Desc", "Group assets based on their plugin. Each plugin group node will all show the dependent plugins."),
		nullptr)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FPluginDependencyGrouping::~FPluginDependencyGrouping()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FPluginDependencyGrouping::GroupNodes(const TArray<UE::Insights::FTableTreeNodePtr>& Nodes, UE::Insights::FTableTreeNode& ParentGroup, TWeakPtr<UE::Insights::FTable> InParentTable, UE::Insights::IAsyncOperationProgress& InAsyncOperationProgress) const
{
	using namespace UE::Insights;

	ParentGroup.ClearChildren();

	TSharedPtr<FAssetTable> AssetTable = StaticCastSharedPtr<FAssetTable>(InParentTable.Pin());
	check(AssetTable.IsValid());

	TMap<int32, FPluginSimpleGroupNode*> PluginIndexToGroupNodeMap;


	// If we have any root plugins then we'll want to ensure we always show those and we divide root from non-root
	// However, a typical UE project may not make use of this concept so if there are no designated root plugins
	// according to the loaded ucookmeta then only show plugins that actually have assets according to the incoming node list
	TSharedPtr<FAssetTreeNode> RootPlugins = MakeShared<FAssetTreeNode>(FName("Root Plugins"), AssetTable);
	TSharedPtr<FAssetTreeNode> NonRootPlugins = MakeShared<FAssetTreeNode>(FName("Non-Root Plugins"), AssetTable);

	bool bFoundAnyRootPlugins = false;
	for (int32 PluginIndex = 0; PluginIndex < AssetTable->GetNumPlugins(); PluginIndex++)
	{
		const FAssetTablePluginInfo& PluginInfo = AssetTable->GetPluginInfoByIndex(PluginIndex);

		if (PluginInfo.IsRootPlugin())
		{
			FName PluginAndDependenciesGroupName = AssetTable->GetNameForPlugin(PluginIndex);
			TSharedPtr<FPluginAndDependenciesGroupNode> PluginAndDependenciesGroup = MakeShared<FPluginAndDependenciesGroupNode>(PluginAndDependenciesGroupName, AssetTable, PluginIndex);
			PluginAndDependenciesGroup->SetAuthorGrouping(this);
			FPluginSimpleGroupNode* PluginGroup = PluginAndDependenciesGroup->CreateChildren().Get();
			PluginIndexToGroupNodeMap.Add(PluginIndex, PluginGroup);
			RootPlugins->AddChildAndSetParent(PluginAndDependenciesGroup);
			bFoundAnyRootPlugins = true;
		}
	}

	if (bFoundAnyRootPlugins)
	{
		ParentGroup.AddChildAndSetParent(RootPlugins);
		ParentGroup.AddChildAndSetParent(NonRootPlugins);
	}
	else
	{
		RootPlugins.Reset();
		NonRootPlugins.Reset();
	}

	for (FTableTreeNodePtr NodePtr : Nodes)
	{
		if (InAsyncOperationProgress.ShouldCancelAsyncOp())
		{
			return;
		}

		if (NodePtr->IsGroup() || !NodePtr->Is<FAssetTreeNode>())
		{
			ParentGroup.AddChildAndSetParent(NodePtr);
			continue;
		}

		FAssetTreeNode& AssetNode = NodePtr->As<FAssetTreeNode>();
		const FAssetTableRow& Asset = AssetNode.GetAssetChecked();

		const TCHAR* PluginName = Asset.GetPluginName();
		const FAssetTablePluginInfo& PluginInfo = AssetTable->GetOrCreatePluginInfo(PluginName);
		int32 PluginIndex = AssetTable->GetIndexForPlugin(PluginName);

		FPluginSimpleGroupNode* PluginGroup = PluginIndexToGroupNodeMap.FindRef(PluginIndex);
		if (PluginGroup)
		{
			// Plugin group node (simple or with dependencies) was already created. Just add the current asset to it.
			PluginGroup->AddChildAndSetParent(NodePtr);
		}
		else
		{
			// Create the Plugin group node (with "Assets" and "Dependencies" sub-groups)
			// and add the current asset to the "Assets" sub-group.
			FName PluginAndDependenciesGroupName = AssetTable->GetNameForPlugin(PluginIndex);
			TSharedPtr<FPluginAndDependenciesGroupNode> PluginAndDependenciesGroup = MakeShared<FPluginAndDependenciesGroupNode>(PluginAndDependenciesGroupName, AssetTable, PluginIndex);
			PluginAndDependenciesGroup->SetAuthorGrouping(this);
			PluginGroup = PluginAndDependenciesGroup->CreateChildren().Get();
			PluginGroup->AddChildAndSetParent(NodePtr);
			PluginIndexToGroupNodeMap.Add(PluginIndex, PluginGroup);

			// A root plugin never has assets (by definition), so don't
			if (NonRootPlugins.IsValid())
			{
				if (PluginInfo.IsRootPlugin())
				{
					UE_LOGF(LogInsights, Error, "Plugin %ls contains assets but is marked as a root plugin.", PluginName);
				}
				NonRootPlugins->AddChildAndSetParent(PluginAndDependenciesGroup);
			}
			else
			{
				ParentGroup.AddChildAndSetParent(PluginAndDependenciesGroup);
			}
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
