// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InsightsCore/Table/ViewModels/BaseTreeNode.h"
#include "InsightsCore/Table/ViewModels/TreeNodeGrouping.h"

class FAssetDependencyGrouping : public UE::Insights::FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FAssetDependencyGrouping, UE::Insights::FTreeNodeGrouping)

public:
	FAssetDependencyGrouping();
	virtual ~FAssetDependencyGrouping();

	virtual void GroupNodes(const TArray<UE::Insights::FTableTreeNodePtr>& Nodes, UE::Insights::FTableTreeNode& ParentGroup, TWeakPtr<UE::Insights::FTable> InParentTable, UE::Insights::IAsyncOperationProgress& InAsyncOperationProgress) const override;
};

class FPluginDependencyGrouping : public UE::Insights::FTreeNodeGrouping
{
	INSIGHTS_DECLARE_RTTI(FPluginDependencyGrouping, UE::Insights::FTreeNodeGrouping)

public:
	FPluginDependencyGrouping();
	virtual ~FPluginDependencyGrouping();

	virtual void GroupNodes(const TArray<UE::Insights::FTableTreeNodePtr>& Nodes, UE::Insights::FTableTreeNode& ParentGroup, TWeakPtr<UE::Insights::FTable> InParentTable, UE::Insights::IAsyncOperationProgress& InAsyncOperationProgress) const override;
};
