// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTable.h"

#include "Styling/StyleColors.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

// AssetManagerEditor
#include "AssetTreeNode.h"
#include "InsightsCore/Common/Log.h"
#include "InsightsCore/Common/Stopwatch.h"
#include "InsightsCore/Table/ViewModels/TableCellValueFormatter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueGetter.h"
#include "InsightsCore/Table/ViewModels/TableCellValueSorter.h"
#include "InsightsCore/Table/ViewModels/TableColumn.h"

#define LOCTEXT_NAMESPACE "FAssetTable"

////////////////////////////////////////////////////////////////////////////////////////////////////
// Column identifiers

const FName FAssetTableColumns::CountColumnId(TEXT("Count"));
const FName FAssetTableColumns::TypeColumnId(TEXT("Type"));
const FName FAssetTableColumns::NameColumnId(TEXT("Name"));
const FName FAssetTableColumns::PathColumnId(TEXT("Path"));
const FName FAssetTableColumns::PrimaryTypeColumnId(TEXT("PrimaryType"));
const FName FAssetTableColumns::PrimaryNameColumnId(TEXT("PrimaryName"));
const FName FAssetTableColumns::StagedCompressedSizeRequiredInstallColumnId(TEXT("StagedCompressedSizeRequiredInstall"));
const FName FAssetTableColumns::TotalSizeUniqueDependenciesColumnId(TEXT("TotalSizeUniqueDependencies"));
const FName FAssetTableColumns::TotalSizeSharedDependenciesColumnId(TEXT("TotalSizeSharedDependencies"));
const FName FAssetTableColumns::TotalSizeExternalDependenciesColumnId(TEXT("TotalSizeExternalDependencies"));
const FName FAssetTableColumns::TotalUsageCountColumnId(TEXT("TotalUsageCount"));
const FName FAssetTableColumns::ChunksColumnId(TEXT("Chunks"));
const FName FAssetTableColumns::NativeClassColumnId(TEXT("NativeClass"));
const FName FAssetTableColumns::PluginNameColumnId(TEXT("PluginName"));
const FName FAssetTableColumns::PluginInclusiveSizeColumnId(TEXT("PluginInclusiveSize"));
const FName FAssetTableColumns::PluginTypeColumnId(TEXT("PluginType"));

////////////////////////////////////////////////////////////////////////////////////////////////////
// FAssetTableStringValueGetterWithDependencyAggregationHandling
////////////////////////////////////////////////////////////////////////////////////////////////////

// A special value getter that handles the common case where a dependency grouping node is acting as a
// proxy for a single asset and its dependencies.
class FAssetTableStringValueGetterWithDependencyAggregationHandling : public UE::Insights::FTableCellValueGetter
{
public:

	virtual const TCHAR* GetStringValueFromAssetTableRow(const FAssetTableRow& Asset) const = 0;

	virtual const TOptional<UE::Insights::FTableCellValue> GetValue(const UE::Insights::FTableColumn& Column, const UE::Insights::FBaseTreeNode& Node) const override
	{
		using namespace UE::Insights;
		const FTableTreeNode& NodePtr = static_cast<const FTableTreeNode&>(Node);
		if (Node.IsGroup() && !NodePtr.GetRowId().HasValidIndex())
		{
			if (NodePtr.HasAggregatedValue(Column.GetId()))
			{
				return NodePtr.GetAggregatedValue(Column.GetId());
			}
		}
		else //if (Node->Is<FAssetTreeNode>())
		{
			TSharedPtr<FAssetTable> AssetTable = StaticCastSharedPtr<FAssetTable>(NodePtr.GetParentTable().Pin());
			const FAssetTableRow& Asset = AssetTable->GetAssetChecked(NodePtr.GetRowId().RowIndex);
			return FTableCellValue(GetStringValueFromAssetTableRow(Asset));
		}

		return TOptional<FTableCellValue>();
	}
};

////////////////////////////////////////////////////////////////////////////////////////////////////
// FAssetTableStringStore
////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetTableStringStore::FAssetTableStringStore()
	: Chunks()
	, Cache()
	, TotalInputStringSize(0)
	, TotalStoredStringSize(0)
	, NumInputStrings(0)
	, NumStoredStrings(0)
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetTableStringStore::~FAssetTableStringStore()
{
	Reset();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetTableStringStore::Reset()
{
	for (FChunk& Chunk : Chunks)
	{
		delete[] Chunk.Buffer;
	}
	Chunks.Reset();
	Cache.Reset();
	TotalInputStringSize = 0;
	TotalStoredStringSize = 0;
	NumInputStrings = 0;
	NumStoredStrings = 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FStringView FAssetTableStringStore::Store(const TCHAR* InStr)
{
	if (!InStr)
	{
		return FStringView();
	}
	return Store(FStringView(InStr));
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FStringView FAssetTableStringStore::Store(const FStringView InStr)
{
	if (InStr.IsEmpty())
	{
		return FStringView();
	}

	check(InStr.Len() <= GetMaxStringLength());

	TotalInputStringSize += (InStr.Len() + 1) * sizeof(TCHAR);
	++NumInputStrings;

	uint32 Hash = GetTypeHash(InStr);

	TArray<FStringView> CachedStrings;
	Cache.MultiFind(Hash, CachedStrings);
	for (const FStringView& CachedString : CachedStrings)
	{
		if (CachedString.Equals(InStr, SearchCase))
		{
			return CachedString;
		}
	}

	if (Chunks.Num() == 0 ||
		Chunks.Last().Used + InStr.Len() + 1 > ChunkBufferLen)
	{
		AddChunk();
	}

	TotalStoredStringSize += (InStr.Len() + 1) * sizeof(TCHAR);
	++NumStoredStrings;

	FChunk& Chunk = Chunks.Last();
	FStringView StoredStr(Chunk.Buffer + Chunk.Used, InStr.Len());
	FMemory::Memcpy((void*)(Chunk.Buffer + Chunk.Used), (const void*)InStr.GetData(), InStr.Len() * sizeof(TCHAR));
	Chunk.Used += InStr.Len();
	Chunk.Buffer[Chunk.Used] = TEXT('\0');
	Chunk.Used ++;
	Cache.Add(Hash, StoredStr);
	return StoredStr;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetTableStringStore::AddChunk()
{
	FChunk& Chunk = Chunks.AddDefaulted_GetRef();
	Chunk.Buffer = new TCHAR[ChunkBufferLen];
	Chunk.Used = 0;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetTableStringStore::EnumerateStrings(TFunction<void(const FStringView Str)> Callback) const
{
	for (const auto& Pair : Cache)
	{
		Callback(Pair.Value);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FAssetTable
////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetTable::FAssetTable()
{
#define RUN_ASSET_TRAVERSAL_TESTS
#ifdef RUN_ASSET_TRAVERSAL_TESTS
	for (int32 i = 0; i < 9; i++)
	{
		FAssetTableRow& Asset = Assets.AddDefaulted_GetRef();
		int32 Id = i;
		int32 Id2 = 10 * i;

		Asset.Type = StoreStr(FString::Printf(TEXT("Type%02d"), Id % 10));
		Asset.Name = StoreStr(FString::Printf(TEXT("Name%d"), Id));
		Asset.Path = StoreStr(FString::Printf(TEXT("A%02d/B%02d/C%02d/D%02d"), Id % 11, Id % 13, Id % 17, Id % 19));
		Asset.PrimaryType = StoreStr(FString::Printf(TEXT("PT_%02d"), Id2 % 10));
		Asset.PrimaryName = StoreStr(FString::Printf(TEXT("PN%d"), Id2));
		Asset.TotalUsageCount = 10;
		Asset.StagedCompressedSizeRequiredInstall = 1;
		Asset.NativeClass = StoreStr(FString::Printf(TEXT("NativeClass%02d"), (Id * Id * Id) % 8));
		Asset.PluginName = StoreStr(TEXT("MockGFP"));
	}

	for (int32 TestIndex = 0; TestIndex < 5; TestIndex++)
	{
		TArray<TSet<int32>> UniqueDependencies;
		UniqueDependencies.SetNum(Assets.Num());
		TArray<TSet<int32>> SharedDependencies;
		SharedDependencies.SetNum(Assets.Num());
		// Reset dependencies for next test
		for (int32 AssetIndex = 0; AssetIndex < Assets.Num(); AssetIndex++)
		{
			Assets[AssetIndex].Dependencies.Empty();
			Assets[AssetIndex].Referencers.Empty();
		}

		if (TestIndex == 0)
		{
			// 0-->1-->2-->3, 4-->3
			Assets[0].Dependencies.Add(1);
			Assets[1].Dependencies.Add(2);
			Assets[2].Dependencies.Add(3);
			Assets[4].Dependencies.Add(3);
			Assets[1].Referencers.Add(0);
			Assets[2].Referencers.Add(1);
			Assets[3].Referencers.Add(2);
			Assets[3].Referencers.Add(4);

			// Add the expected results in order 0-->4
			UniqueDependencies[0] = TSet<int32>{1, 2};
			UniqueDependencies[1] = TSet<int32>{2};
			UniqueDependencies[2] = TSet<int32>{};
			UniqueDependencies[3] = TSet<int32>{};
			UniqueDependencies[4] = TSet<int32>{};

			SharedDependencies[0] = TSet<int32>{3};
			SharedDependencies[1] = TSet<int32>{3};
			SharedDependencies[2] = TSet<int32>{3};
			SharedDependencies[3] = TSet<int32>{};
			SharedDependencies[4] = TSet<int32>{3};
		}
		else if (TestIndex == 1)
		{
			// 0-->1-->2-->3, 4-->0
			Assets[0].Dependencies.Add(1);
			Assets[1].Dependencies.Add(2);
			Assets[2].Dependencies.Add(3);
			Assets[4].Dependencies.Add(0);
			Assets[1].Referencers.Add(0);
			Assets[2].Referencers.Add(1);
			Assets[3].Referencers.Add(2);
			Assets[0].Referencers.Add(4);

			// Add the expected results in order 0-->4
			UniqueDependencies[0] = TSet<int32>{1, 2, 3};
			UniqueDependencies[1] = TSet<int32>{2, 3};
			UniqueDependencies[2] = TSet<int32>{3};
			UniqueDependencies[3] = TSet<int32>{};
			UniqueDependencies[4] = TSet<int32>{0, 1, 2, 3};

			SharedDependencies[0] = TSet<int32>{};
			SharedDependencies[1] = TSet<int32>{};
			SharedDependencies[2] = TSet<int32>{};
			SharedDependencies[3] = TSet<int32>{};
			SharedDependencies[4] = TSet<int32>{};
		}
		else if (TestIndex == 2)
		{
			// 0-->1-->2-->3-->0, 4-->2
			Assets[0].Dependencies.Add(1);
			Assets[1].Dependencies.Add(2);
			Assets[2].Dependencies.Add(3);
			Assets[3].Dependencies.Add(0);
			Assets[4].Dependencies.Add(2);
			Assets[1].Referencers.Add(0);
			Assets[2].Referencers.Add(1);
			Assets[2].Referencers.Add(4);
			Assets[3].Referencers.Add(2);
			Assets[0].Referencers.Add(3);

			// Add the expected results in order 0-->4
			UniqueDependencies[0] = TSet<int32>{1};
			UniqueDependencies[1] = TSet<int32>{};
			UniqueDependencies[2] = TSet<int32>{3, 0, 1};
			UniqueDependencies[3] = TSet<int32>{0, 1};

			// This is an interesting result. When we traverse the graph for Element 4
			// we find 4-->2-->3-->0-->1-->[terminate loop]. From that point of view,
			// '2' is not shared because its other referencer is part of 4's dependency chain
			// therefore everything is treated as a unique dependency of 4.
			UniqueDependencies[4] = TSet<int32>{2, 3, 0, 1};

			SharedDependencies[0] = TSet<int32>{2, 3};
			SharedDependencies[1] = TSet<int32>{2, 3, 0};
			SharedDependencies[2] = TSet<int32>{};
			SharedDependencies[3] = TSet<int32>{2};
			SharedDependencies[4] = TSet<int32>{};
		}
		else if (TestIndex == 3)
		{
			// 0-->1-->2-->3-->0, 4-->2, 5-->2
			Assets[0].Dependencies.Add(1);
			Assets[1].Dependencies.Add(2);
			Assets[2].Dependencies.Add(3);
			Assets[3].Dependencies.Add(0);
			Assets[4].Dependencies.Add(2);
			Assets[5].Dependencies.Add(2);
			Assets[1].Referencers.Add(0);
			Assets[2].Referencers.Add(1);
			Assets[2].Referencers.Add(4);
			Assets[2].Referencers.Add(5);
			Assets[3].Referencers.Add(2);
			Assets[0].Referencers.Add(3);

			// Add the expected results in order 0-->4
			UniqueDependencies[0] = TSet<int32>{ 1 };
			UniqueDependencies[1] = TSet<int32>{};
			UniqueDependencies[2] = TSet<int32>{ 3, 0, 1 };
			UniqueDependencies[3] = TSet<int32>{ 0, 1 };

			// Unlike the previous test, since 5 also references 2,
			// 4 will see itself as having no unique dependencies
			UniqueDependencies[4] = TSet<int32>{};
			UniqueDependencies[5] = TSet<int32>{};

			SharedDependencies[0] = TSet<int32>{ 2, 3 };
			SharedDependencies[1] = TSet<int32>{ 2, 3, 0 };
			SharedDependencies[2] = TSet<int32>{};
			SharedDependencies[3] = TSet<int32>{ 2 };
			SharedDependencies[4] = TSet<int32>{2, 3, 0, 1};
			SharedDependencies[5] = TSet<int32>{2, 3, 0, 1};
		}
		else if (TestIndex == 4)
		{
			// 0 --> 1 --> 6
			// 0 --> 2 --> 6
			// 0 --> 7
			// 0 --> 5 --> 8
			// 3 --> 2 --> 6
			// 3 --> 5 --> 8
			// 2 --> 5 --> 8
			// 3 --> 4
			// 2 --> 8

			Assets[0].Dependencies.Add(1);
			Assets[0].Dependencies.Add(2);
			Assets[1].Dependencies.Add(6);
			Assets[0].Dependencies.Add(7);
			Assets[3].Dependencies.Add(2);
			Assets[2].Dependencies.Add(6);
			Assets[3].Dependencies.Add(5);
			Assets[2].Dependencies.Add(5);
			Assets[0].Dependencies.Add(5);
			Assets[3].Dependencies.Add(4);
			Assets[5].Dependencies.Add(8);
			Assets[2].Dependencies.Add(8);

			Assets[1].Referencers.Add(0);
			Assets[2].Referencers.Add(0);
			Assets[6].Referencers.Add(1);
			Assets[7].Referencers.Add(0);
			Assets[2].Referencers.Add(3);
			Assets[6].Referencers.Add(2);
			Assets[5].Referencers.Add(3);
			Assets[5].Referencers.Add(2);
			Assets[5].Referencers.Add(0);
			Assets[4].Referencers.Add(3);
			Assets[8].Referencers.Add(2);
			Assets[8].Referencers.Add(5);

			UniqueDependencies[0] = TSet<int32>{ 1, 7 };
			UniqueDependencies[1] = TSet<int32>{};
			UniqueDependencies[2] = TSet<int32>{};
			UniqueDependencies[3] = TSet<int32>{ 4 };
			UniqueDependencies[4] = TSet<int32>{};
			UniqueDependencies[5] = TSet<int32>{};
			UniqueDependencies[6] = TSet<int32>{};
			UniqueDependencies[7] = TSet<int32>{};
			UniqueDependencies[8] = TSet<int32>{};

			SharedDependencies[0] = TSet<int32>{ 2, 6, 5, 8 };
			SharedDependencies[1] = TSet<int32>{ 6 };
			SharedDependencies[2] = TSet<int32>{ 5, 6, 8 };
			SharedDependencies[3] = TSet<int32>{ 2, 5, 8, 6 };
			SharedDependencies[4] = TSet<int32>{};
			SharedDependencies[5] = TSet<int32>{ 8 };
			SharedDependencies[6] = TSet<int32>{};
			SharedDependencies[7] = TSet<int32>{};
			SharedDependencies[8] = TSet<int32>{};

		}

		for (int32 AssetIndex = 0; AssetIndex < Assets.Num(); AssetIndex++)
		{
			TSet<int32> DiscoveredUniqueDependencies;
			TSet<int32> DiscoveredSharedDependencies;
			Assets[AssetIndex].ComputeDependencySizes(*this, TSet<int32>{AssetIndex}, &DiscoveredUniqueDependencies, &DiscoveredSharedDependencies);
			ensureAlways(DiscoveredUniqueDependencies.Num() == UniqueDependencies[AssetIndex].Num()
				&& DiscoveredUniqueDependencies.Intersect(UniqueDependencies[AssetIndex]).Num() == DiscoveredUniqueDependencies.Num());
			ensureAlways(DiscoveredSharedDependencies.Num() == SharedDependencies[AssetIndex].Num()
				&& DiscoveredSharedDependencies.Intersect(SharedDependencies[AssetIndex]).Num() == DiscoveredSharedDependencies.Num());
		}
	}
	ClearAllData();
#endif
#if 0 // debug, mock data

	VisibleAssetCount = 100;
	constexpr int32 HiddenAssetCount = 50;
	const int32 TotalAssetCount = VisibleAssetCount + HiddenAssetCount;

	// Create assets.
	Assets.Reserve(TotalAssetCount);
	for (int32 AssetIndex = 0; AssetIndex < TotalAssetCount; ++AssetIndex)
	{
		FAssetTableRow& Asset = Assets.AddDefaulted_GetRef();

		int32 Id = FMath::Rand();
		int32 Id2 = FMath::Rand();

		Asset.Type = FString::Printf(TEXT("Type%02d"), Id % 10);
		Asset.Name = FString::Printf(TEXT("Name%d"), Id);
		Asset.Path = FString::Printf(TEXT("A%02d/B%02d/C%02d/D%02d"), Id % 11, Id % 13, Id % 17, Id % 19);
		Asset.PrimaryType = FString::Printf(TEXT("PT_%02d"), Id2 % 10);
		Asset.PrimaryName = FString::Printf(TEXT("PN%d"), Id2);
		//Asset.ManagedDiskSize = FMath::Abs(Id * Id);
		//Asset.DiskSize = FMath::Abs(Id * Id * Id);
		Asset.StagedCompressedSize = Asset.DiskSize / 2;
		Asset.TotalUsageCount = Id % 1000;
		//Asset.CookRule = FString::Printf(TEXT("CookRule%02d"), (Id * Id) % 8);
		//Asset.Chunks = FString::Printf(TEXT("Chunks%02d"), (Id * Id + 1) % 41);
		Asset.NativeClass = FString::Printf(TEXT("NativeClass%02d"), (Id * Id * Id) % 8);
		Asset.GameFeaturePlugin = FString::Printf(TEXT("GFP_%02d"), (Id * Id) % 7);
	}

	// Set dependencies (only for visible assets)
	for (int32 AssetIndex = 0; AssetIndex < VisibleAssetCount; ++AssetIndex)
	{
		if (FMath::Rand() % 100 > 60) // 60% chance to have dependencies
		{
			continue;
		}

		FAssetTableRow& Asset = Assets[AssetIndex];

		int32 NumDependents = FMath::Rand() % 30; // max 30 dependent assets
		while (--NumDependents >= 0)
		{
			int32 DepIndex = -1;
			if (FMath::Rand() % 100 <= 10) // 10% chance to be another asset that is visible by default
			{
				DepIndex = FMath::Rand() % VisibleAssetCount;
			}
			else // 90% chance to be a dependet only asset (not visible by default)
			{
				DepIndex = VisibleAssetCount + FMath::Rand() % HiddenAssetCount;
			}
			if (!Asset.Dependencies.Contains(DepIndex))
			{
				Asset.Dependencies.Add(DepIndex);
			}
		}
	}

#endif
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetTable::~FAssetTable()
{
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetTable::Reset()
{
	//...

	FTable::Reset();

	AddDefaultColumns();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetTable::EnumeratePluginDependencies(const FAssetTablePluginInfo& InPlugin, TFunction<void(int32 PluginIndex)> Callback) const
{
	for (int32 DependencyIndex : InPlugin.PluginDependencies)
	{
		Callback(DependencyIndex);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetTable::EnumerateAssetsForPlugin(const FAssetTablePluginInfo& InPlugin, TFunction<void(int32 AssetIndex)> Callback) const
{
	TSet<int32> PluginAssets = InPlugin.GetPluginAssets();
	for (int32 AssetIndex : PluginAssets)
	{
		Callback(AssetIndex);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetTable::PopulateAssetsInPlugins()
{
	for (int32 AssetIndex = 0 ; AssetIndex < Assets.Num() ; AssetIndex++)
	{
		FAssetTablePluginInfo& PluginInfo = GetOrCreatePluginInfo(Assets[AssetIndex].GetPluginName());
		PluginInfo.AddPluginAsset(AssetIndex);
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetTablePluginInfo& FAssetTable::GetOrCreatePluginInfo(const TCHAR* StoredPluginName)
{
	if (int32* PluginIndex = PluginNameToIndexMap.Find(StoredPluginName))
	{
		return Plugins[*PluginIndex];
	}
	else
	{
		FAssetTablePluginInfo NewPlugin;
		NewPlugin.PluginName = StoredPluginName;
		int32 NewIndex = Plugins.Num();
		Plugins.Add(NewPlugin);
		PluginNameToIndexMap.Add(StoredPluginName, NewIndex);
		return Plugins[NewIndex];
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int32 FAssetTable::GetIndexForPlugin(const TCHAR* StoredPluginName) const
{
	if (const int32* Index = PluginNameToIndexMap.Find(StoredPluginName))
	{
		return *Index;
	}
	return -1;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetTable::AddDefaultColumns()
{
	using namespace UE::Insights;

	//////////////////////////////////////////////////
	// Hierarchy Column
	{
		const int32 HierarchyColumnIndex = -1;
		const TCHAR* HierarchyColumnName = nullptr;
		AddHierarchyColumn(HierarchyColumnIndex, HierarchyColumnName);

		const TSharedRef<FTableColumn>& ColumnRef = GetColumns()[0];
		ColumnRef->SetInitialWidth(400.0f);
		ColumnRef->SetShortName(LOCTEXT("HierarchyColumnName", "Hierarchy"));
		ColumnRef->SetTitleName(LOCTEXT("HierarchyColumnTitle", "Asset Hierarchy"));
		ColumnRef->SetDescription(LOCTEXT("HierarchyColumnDesc", "Hierarchy of the asset tree"));
	}

	int32 ColumnIndex = 0;

	//////////////////////////////////////////////////
	// Count Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::CountColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("CountColumnName", "Count"));
		Column.SetTitleName(LOCTEXT("CountColumnTitle", "Asset Count"));
		Column.SetDescription(LOCTEXT("CountColumnDesc", "Number of assets"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered | ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FAssetCountValueGetter : public FTableCellValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.IsGroup())
				{
					const FTableTreeNode& NodePtr = static_cast<const FTableTreeNode&>(Node);
					if (NodePtr.HasAggregatedValue(Column.GetId()))
					{
						return NodePtr.GetAggregatedValue(Column.GetId());
					}
				}
				else //if (Node->Is<FAssetTreeNode>())
				{
					//const FAssetTreeNode& TreeNode = static_cast<const FAssetTreeNode&>(Node);
					//const FAssetTableRow& Asset = TreeNode.GetAssetChecked();
					return FTableCellValue((int64)1);
				}

				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAssetCountValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}

	//////////////////////////////////////////////////
	// Staged Compressed Size Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::StagedCompressedSizeRequiredInstallColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("StagedCompressedSizeRequiredInstallColumnName", "Self Size"));
		Column.SetTitleName(LOCTEXT("StagedCompressedSizeRequiredInstallColumnTitle", "Self Size (Compressed, Required Install)"));
		Column.SetDescription(LOCTEXT("StagedCompressedSizeRequiredInstallColumnDesc", "Compressed size of required install iostore chunks for this asset's package. Only visible after staging."));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered | ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FStagedCompressedSizeValueGetter : public FTableCellValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FPluginSimpleGroupNode>() && !Node.Is<FPluginDependenciesGroupNode>())
				{
					// This is node represents a single plugin (it might be the plugin itself or the plugin+deps node for that plugin)
					const FPluginSimpleGroupNode& PluginNode = Node.As<FPluginSimpleGroupNode>();
					TSharedPtr<FTable> TablePtr = PluginNode.GetParentTable().Pin();
					const FAssetTable& AssetTable = static_cast<const FAssetTable&>(*TablePtr);
					const FAssetTablePluginInfo& PluginInfo = AssetTable.GetPluginInfoByIndexChecked(PluginNode.GetPluginIndex());
					return FTableCellValue(PluginInfo.GetSize());
				}
				else if (Node.Is<FAssetTreeNode>() && Node.As<FAssetTreeNode>().IsValidAsset())
				{
					if (Node.Is<FAssetDependenciesGroupTreeNode>())
					{
						return TOptional<FTableCellValue>();
					}
					const FAssetTreeNode& TreeNode = Node.As<FAssetTreeNode>();
					const FAssetTableRow& Asset = TreeNode.GetAssetChecked();
					return FTableCellValue(static_cast<int64>(Asset.GetStagedCompressedSizeRequiredInstall()));
				}
				else if (Node.IsGroup() && !Node.Is<FPluginDependenciesGroupNode>())
				{
					const FTableTreeNode& NodePtr = static_cast<const FTableTreeNode&>(Node);
					if (NodePtr.HasAggregatedValue(Column.GetId()))
					{
						return NodePtr.GetAggregatedValue(Column.GetId());
					}
				}

				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FStagedCompressedSizeValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		//TSharedRef<IFilterValueConverter> Converter = MakeShared<FMemoryFilterValueConverter>();
		//Column.SetValueConverter(Converter);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}

	//////////////////////////////////////////////////
	// PluginInclusiveSize Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::PluginInclusiveSizeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("PluginInclusiveSizeColumnName", "Incl. Size Plugin"));
		Column.SetTitleName(LOCTEXT("PluginInclusiveSizeColumnTitle", "Plugin Inclusive Size"));
		Column.SetDescription(LOCTEXT("PluginInclusiveColumnDesc", "Inclusive size of this plugin and its dependencies"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(50.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FPluginInclusiveSizeValueGetter : public FTableCellValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FPluginSimpleGroupNode>())
				{
					TSharedPtr<FTable> TablePtr = static_cast<const FPluginSimpleGroupNode&>(Node).GetParentTable().Pin();
					const FAssetTable& AssetTable = static_cast<const FAssetTable&>(*TablePtr);
					int32 PluginIndex = static_cast<const FPluginSimpleGroupNode&>(Node).GetPluginIndex();
					return FTableCellValue(AssetTable.GetPluginInfoByIndex(PluginIndex).GetOrComputeTotalSizeInclusiveOfDependencies(AssetTable));
				}

				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPluginInclusiveSizeValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		Column.SetAggregation(ETableColumnAggregation::None);

		AddColumn(ColumnRef);
	}

	//////////////////////////////////////////////////
	// Type Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::TypeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TypeColumnName", "Type"));
		Column.SetTitleName(LOCTEXT("TypeColumnTitle", "Type"));
		Column.SetDescription(LOCTEXT("TypeColumnDesc", "Asset's type"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(200.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FAssetTypeValueGetter : public FAssetTableStringValueGetterWithDependencyAggregationHandling 
		{
		public:

			virtual const TCHAR* GetStringValueFromAssetTableRow(const FAssetTableRow& Asset) const override
			{
				return Asset.GetType();
			}
		};

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAssetTypeValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}

	//////////////////////////////////////////////////
	// Name Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::NameColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("NameColumnName", "Name"));
		Column.SetTitleName(LOCTEXT("NameColumnTitle", "Name"));
		Column.SetDescription(LOCTEXT("NameColumnDesc", "Asset's name"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(200.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FAssetNameValueGetter : public FAssetTableStringValueGetterWithDependencyAggregationHandling
		{
		public:

			virtual const TCHAR* GetStringValueFromAssetTableRow(const FAssetTableRow& Asset) const override
			{
				return Asset.GetName();
			}
		};

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAssetNameValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Path Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::PathColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("PathColumnName", "Path"));
		Column.SetTitleName(LOCTEXT("PathColumnTitle", "Path"));
		Column.SetDescription(LOCTEXT("PathColumnDesc", "Asset's path"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(200.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FAssetPathValueGetter : public FAssetTableStringValueGetterWithDependencyAggregationHandling
		{
		public:

			virtual const TCHAR* GetStringValueFromAssetTableRow(const FAssetTableRow& Asset) const override
			{
				return Asset.GetPath();
			}
		};

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAssetPathValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Primary Type Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::PrimaryTypeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("PrimaryTypeColumnName", "Primary Type"));
		Column.SetTitleName(LOCTEXT("PrimaryTypeColumnTitle", "Primary Type"));
		Column.SetDescription(LOCTEXT("PrimaryTypeColumnDesc", "Primary Asset Type of this asset, if set"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(200.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FAssetPrimaryTypeValueGetter : public FAssetTableStringValueGetterWithDependencyAggregationHandling
		{
		public:

			virtual const TCHAR* GetStringValueFromAssetTableRow(const FAssetTableRow& Asset) const override
			{
				return Asset.GetPrimaryType();
			}
		};

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAssetPrimaryTypeValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Primary Name Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::PrimaryNameColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("PrimaryNameColumnName", "Primary Name"));
		Column.SetTitleName(LOCTEXT("PrimaryNameColumnTitle", "Primary Name"));
		Column.SetDescription(LOCTEXT("PrimaryNameColumnDesc", "Primary Asset Name of this asset, if set"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(200.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FAssetPrimaryNameValueGetter : public FAssetTableStringValueGetterWithDependencyAggregationHandling
		{
		public:

			virtual const TCHAR* GetStringValueFromAssetTableRow(const FAssetTableRow& Asset) const override
			{
				return Asset.GetPrimaryName();
			}
		};

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FAssetPrimaryNameValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	
	//////////////////////////////////////////////////
	// Total Size of Unique Dependencies
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::TotalSizeUniqueDependenciesColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TotalSizeUniqueDependenciesColumnName", "Unique"));
		Column.SetTitleName(LOCTEXT("TotalSizeUniqueDependenciesColumnTitle", "Total Unique Dependency Size"));
		Column.SetDescription(LOCTEXT("TotalSizeUniqueDependenciesColumnIdDesc", "Sum of the staged compressed sizes of all dependencies of this item, counted only once"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FTotalSizeUniqueDependenciesValueGetter : public FTableCellValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FPluginSimpleGroupNode>() && !Node.Is<FPluginDependenciesGroupNode>())
				{
					// This is node represents a single plugin (it might be the plugin itself or the plugin+deps node for that plugin)
					const FPluginSimpleGroupNode& PluginNode = Node.As<FPluginSimpleGroupNode>();
					TSharedPtr<FTable> TablePtr = PluginNode.GetParentTable().Pin();
					const FAssetTable& AssetTable = static_cast<const FAssetTable&>(*TablePtr);
					const FAssetTablePluginInfo& PluginInfo = AssetTable.GetPluginInfoByIndexChecked(PluginNode.GetPluginIndex());
					return FTableCellValue(PluginInfo.GetOrComputeTotalSizeUniqueDependencies(static_cast<const FAssetTable&>(*TablePtr)));
				}
				else if (Node.IsGroup())
				{
					const FTableTreeNode& NodePtr = static_cast<const FTableTreeNode&>(Node);
					if (NodePtr.GetRowId().HasValidIndex())
					{
						// This group node also has a row index, so it conceptually only represents a single row (i.e., it's a synthetic group created by the asset dependency grouping)
						TSharedPtr<FTable> TablePtr = NodePtr.GetParentTable().Pin();
						const FAssetTable& AssetTable = static_cast<const FAssetTable&>(*TablePtr);
						return FTableCellValue(static_cast<int64>(AssetTable.GetAssetChecked(NodePtr.GetRowIndex()).GetOrComputeTotalSizeUniqueDependencies(AssetTable, NodePtr.GetRowIndex())));
					}
					else if (NodePtr.HasAggregatedValue(Column.GetId()))
					{
						return NodePtr.GetAggregatedValue(Column.GetId());
					}
				}
				else //if (Node->Is<FAssetTreeNode>())
				{
					const FAssetTreeNode& TreeNode = static_cast<const FAssetTreeNode&>(Node);
					const FAssetTableRow& Asset = TreeNode.GetAssetChecked();
					return FTableCellValue(static_cast<int64>(Asset.GetOrComputeTotalSizeUniqueDependencies(TreeNode.GetAssetTableChecked(), TreeNode.GetRowIndex())));
				}

				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTotalSizeUniqueDependenciesValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		// Disable sorting on this column for now as it's quite expensive
		//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		//Column.SetValueSorter(Sorter);
		//Column.SetInitialSortMode(EColumnSortMode::Descending);

		Column.SetAggregation(ETableColumnAggregation::None);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Total Size of Shared Dependencies
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::TotalSizeSharedDependenciesColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TotalSizeSharedDependenciesColumnName", "Shared"));
		Column.SetTitleName(LOCTEXT("TotalSizeSharedDependenciesColumnTitle", "Total Shared Dependency Size"));
		Column.SetDescription(LOCTEXT("TotalSizeSharedDependenciesColumnIdDesc", "Sum of the staged compressed sizes of all dependencies of this asset which are shared by other assets directly or indirectly, counted only once"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FTotalSizeSharedDependenciesValueGetter : public FTableCellValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.IsGroup())
				{
					const FTableTreeNode& NodePtr = static_cast<const FTableTreeNode&>(Node);
					if (NodePtr.GetRowId().HasValidIndex())
					{
						// This group node also has a row index, so it conceptually only represents a single row (i.e., it's a synthetic group created by the asset dependency grouping)
						TSharedPtr<FTable> TablePtr = NodePtr.GetParentTable().Pin();
						const FAssetTable& AssetTable = static_cast<const FAssetTable&>(*TablePtr);
						return FTableCellValue(static_cast<int64>(AssetTable.GetAssetChecked(NodePtr.GetRowIndex()).GetOrComputeTotalSizeSharedDependencies(AssetTable, NodePtr.GetRowIndex())));
					}
					else if (NodePtr.HasAggregatedValue(Column.GetId()))
					{
						return NodePtr.GetAggregatedValue(Column.GetId());
					}
				}
				else //if (Node->Is<FAssetTreeNode>())
				{
					const FAssetTreeNode& TreeNode = static_cast<const FAssetTreeNode&>(Node);
					const FAssetTableRow& Asset = TreeNode.GetAssetChecked();
					return FTableCellValue(static_cast<int64>(Asset.GetOrComputeTotalSizeSharedDependencies(TreeNode.GetAssetTableChecked(), TreeNode.GetRowIndex())));
				}

				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTotalSizeSharedDependenciesValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		// Disable sorting on this column for now as it's quite expensive
		//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		//Column.SetValueSorter(Sorter);
		//Column.SetInitialSortMode(EColumnSortMode::Descending);

		Column.SetAggregation(ETableColumnAggregation::None);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Total Size of External Dependencies
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::TotalSizeExternalDependenciesColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TotalSizeExternalDependenciesColumnName", "External"));
		Column.SetTitleName(LOCTEXT("TotalSizeExternalDependenciesColumnTitle", "Total External Dependency Size"));
		Column.SetDescription(LOCTEXT("TotalSizeExternalDependenciesColumnIdDesc", "Sum of the staged compressed sizes of all dependencies of this asset which are shared by other assets directly or indirectly and exist OUTSIDE the GFP of this asset, counted only once"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FTotalSizeExternalDependenciesValueGetter : public FTableCellValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.IsGroup())
				{
					const FTableTreeNode& NodePtr = static_cast<const FTableTreeNode&>(Node);
					if (NodePtr.GetRowId().HasValidIndex())
					{
						// This group node also has a row index, so it conceptually only represents a single row (i.e., it's a synthetic group created by the asset dependency grouping)
						TSharedPtr<FTable> TablePtr = NodePtr.GetParentTable().Pin();
						const FAssetTable& AssetTable = static_cast<const FAssetTable&>(*TablePtr);
						return FTableCellValue(static_cast<int64>(AssetTable.GetAssetChecked(NodePtr.GetRowIndex()).GetOrComputeTotalSizeExternalDependencies(AssetTable, NodePtr.GetRowIndex())));
					}
					else if (NodePtr.HasAggregatedValue(Column.GetId()))
					{
						return NodePtr.GetAggregatedValue(Column.GetId());
					}
				}
				else //if (Node->Is<FAssetTreeNode>())
				{
					const FAssetTreeNode& TreeNode = static_cast<const FAssetTreeNode&>(Node);
					const FAssetTableRow& Asset = TreeNode.GetAssetChecked();
					return FTableCellValue(static_cast<int64>(Asset.GetOrComputeTotalSizeExternalDependencies(TreeNode.GetAssetTableChecked(), TreeNode.GetRowIndex())));
				}

				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTotalSizeExternalDependenciesValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsMemory>();
		Column.SetValueFormatter(Formatter);

		// Disable sorting on this column for now as it's quite expensive
		//TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		//Column.SetValueSorter(Sorter);
		//Column.SetInitialSortMode(EColumnSortMode::Descending);

		Column.SetAggregation(ETableColumnAggregation::None);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Total Usage Count Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::TotalUsageCountColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("TotalUsageCountColumnName", "Total Usage"));
		Column.SetTitleName(LOCTEXT("TotalUsageCountColumnTitle", "Total Usage Count"));
		Column.SetDescription(LOCTEXT("TotalUsageCountColumnDesc", "Weighted count of Primary Assets that use this\nA higher usage means it's more likely to be in memory at runtime."));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered | ETableColumnFlags::IsDynamic);

		Column.SetHorizontalAlignment(HAlign_Right);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::Int64);

		class FTotalUsageCountValueGetter : public FTableCellValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.IsGroup())
				{
					const FTableTreeNode& NodePtr = static_cast<const FTableTreeNode&>(Node);
					if (NodePtr.HasAggregatedValue(Column.GetId()))
					{
						return NodePtr.GetAggregatedValue(Column.GetId());
					}
				}
				else //if (Node->Is<FAssetTreeNode>())
				{
					const FAssetTreeNode& TreeNode = static_cast<const FAssetTreeNode&>(Node);
					const FAssetTableRow& Asset = TreeNode.GetAssetChecked();
					return FTableCellValue(static_cast<int64>(Asset.GetTotalUsageCount()));
				}

				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FTotalUsageCountValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FInt64ValueFormatterAsNumber>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByInt64Value>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		Column.SetAggregation(ETableColumnAggregation::Sum);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Chunks Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::ChunksColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("ChunksColumnName", "Chunks"));
		Column.SetTitleName(LOCTEXT("ChunksColumnTitle", "Chunks"));
		Column.SetDescription(LOCTEXT("ChunksColumnDesc", "List of chunks this will be added to when cooked"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(100.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FChunksValueGetter : public FAssetTableStringValueGetterWithDependencyAggregationHandling
		{
		public:

			virtual const TCHAR* GetStringValueFromAssetTableRow(const FAssetTableRow& Asset) const override
			{
				return Asset.GetChunks();
			}
		};

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FChunksValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// Native Class Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::NativeClassColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("NativeClassColumnName", "Native Class"));
		Column.SetTitleName(LOCTEXT("NativeClassColumnTitle", "Native Class"));
		Column.SetDescription(LOCTEXT("NativeClassColumnDesc", "Native class of the asset"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(200.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FNativeClassValueGetter : public FAssetTableStringValueGetterWithDependencyAggregationHandling
		{
		public:

			virtual const TCHAR* GetStringValueFromAssetTableRow(const FAssetTableRow& Asset) const override
			{
				return Asset.GetNativeClass();
			}
		};

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FNativeClassValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
	//////////////////////////////////////////////////
	// PluginName Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::PluginNameColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("PluginColumnName", "Plugin"));
		Column.SetTitleName(LOCTEXT("PluginColumnTitle", "Plugin"));
		Column.SetDescription(LOCTEXT("PluginColumnDesc", "Plugin (Game Feature or Engine) of the asset"));

		Column.SetFlags(ETableColumnFlags::ShouldBeVisible | ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(200.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FPluginNameValueGetter : public FAssetTableStringValueGetterWithDependencyAggregationHandling
		{
		public:

			virtual const TCHAR* GetStringValueFromAssetTableRow(const FAssetTableRow& Asset) const override
			{
				return Asset.GetPluginName();
			}
		};

		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPluginNameValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}

	//////////////////////////////////////////////////
	// PluginType Column
	{
		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(FAssetTableColumns::PluginTypeColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		Column.SetShortName(LOCTEXT("PluginTypeColumnName", "Plugin Type"));
		Column.SetTitleName(LOCTEXT("PluginTypeColumnTitle", "Plugin Type"));
		Column.SetDescription(LOCTEXT("PluginTypeColumnDesc", "Plugin Type (e.g., Normal, Root, Shader Pseudoplugin, etc.)"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		Column.SetInitialWidth(75.0f);

		Column.SetDataType(ETableCellDataType::CString);

		class FPluginTypeValueGetter : public FTableCellValueGetter
		{
		public:
			virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
			{
				if (Node.Is<FPluginSimpleGroupNode>())
				{
					TSharedPtr<FTable> TablePtr = static_cast<const FPluginSimpleGroupNode&>(Node).GetParentTable().Pin();
					const FAssetTable& AssetTable = static_cast<const FAssetTable&>(*TablePtr);
					int32 PluginIndex = static_cast<const FPluginSimpleGroupNode&>(Node).GetPluginIndex();
					return FTableCellValue(AssetTable.GetPluginInfoByIndex(PluginIndex).GetPluginType());
				}

				return TOptional<FTableCellValue>();
			}
		};
		TSharedRef<ITableCellValueGetter> Getter = MakeShared<FPluginTypeValueGetter>();
		Column.SetValueGetter(Getter);

		TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
		Column.SetValueFormatter(Formatter);

		TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
		Column.SetValueSorter(Sorter);
		Column.SetInitialSortMode(EColumnSortMode::Descending);

		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}

	//////////////////////////////////////////////////
	// Custom Columns
	for (int32 CustomColumnIndex = 0; CustomColumnIndex < CustomColumns.Num(); CustomColumnIndex++)
	{
		const FCustomColumnDefinition& CustomColumn = CustomColumns[CustomColumnIndex];

		TSharedRef<FTableColumn> ColumnRef = MakeShared<FTableColumn>(CustomColumn.ColumnId);
		FTableColumn& Column = *ColumnRef;

		Column.SetIndex(ColumnIndex++);

		const FText NameAsText = FText::FromName(CustomColumn.ColumnId);
		Column.SetShortName(NameAsText);
		Column.SetTitleName(NameAsText);
		Column.SetDescription(LOCTEXT("CustomColumnDesc", "Custom column data"));

		Column.SetFlags(ETableColumnFlags::CanBeHidden | ETableColumnFlags::CanBeFiltered);

		Column.SetHorizontalAlignment(HAlign_Left);
		if (CustomColumn.Type == ECustomColumnDefinitionType::Boolean)
		{
			Column.SetInitialWidth(50.f);
			Column.SetDataType(ETableCellDataType::Bool);
			class FCustomColumnBoolValueGetter : public FTableCellValueGetter
			{
			public:

				FCustomColumnBoolValueGetter(int32 InIndexWithinRowData) :
					FTableCellValueGetter(),
					IndexWithinRowData(InIndexWithinRowData) {}

				virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
				{
					if (Node.Is<FPluginSimpleGroupNode>() && !Node.Is<FPluginDependenciesGroupNode>())
					{
						// This node represents a single plugin (it might be the plugin itself or the plugin+deps node for that plugin)
						const FPluginSimpleGroupNode& PluginNode = Node.As<FPluginSimpleGroupNode>();
						TSharedPtr<FTable> TablePtr = PluginNode.GetParentTable().Pin();
						const FAssetTable& AssetTable = static_cast<const FAssetTable&>(*TablePtr);
						const FAssetTablePluginInfo& PluginInfo = AssetTable.GetPluginInfoByIndexChecked(PluginNode.GetPluginIndex());
						if (const bool* Value = PluginInfo.TryGetDataByKey<bool>(IndexWithinRowData))
						{
							return FTableCellValue(*Value);
						}
					}

					return TOptional<FTableCellValue>();
				}

			private:
				int32 IndexWithinRowData;
			};

			TSharedRef<ITableCellValueGetter> Getter = MakeShared<FCustomColumnBoolValueGetter>(CustomColumnIndex);
			Column.SetValueGetter(Getter);

			TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FBoolValueFormatterAsTrueFalse>();
			Column.SetValueFormatter(Formatter);

			TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByBoolValue>(ColumnRef);
			Column.SetValueSorter(Sorter);
		}
		else
		{
			Column.SetInitialWidth(200.f);
			Column.SetDataType(ETableCellDataType::CString);
			class FCustomColumnStringValueGetter : public FTableCellValueGetter
			{
			public:

				FCustomColumnStringValueGetter(int32 InIndexWithinRowData) :
					FTableCellValueGetter(),
					IndexWithinRowData(InIndexWithinRowData) {}

				virtual const TOptional<FTableCellValue> GetValue(const FTableColumn& Column, const FBaseTreeNode& Node) const override
				{
					if (Node.Is<FPluginSimpleGroupNode>() && !Node.Is<FPluginDependenciesGroupNode>())
					{
						// This node represents a single plugin (it might be the plugin itself or the plugin+deps node for that plugin)
						const FPluginSimpleGroupNode& PluginNode = Node.As<FPluginSimpleGroupNode>();
						TSharedPtr<FTable> TablePtr = PluginNode.GetParentTable().Pin();
						const FAssetTable& AssetTable = static_cast<const FAssetTable&>(*TablePtr);
						const FAssetTablePluginInfo& PluginInfo = AssetTable.GetPluginInfoByIndexChecked(PluginNode.GetPluginIndex());
						if (const TCHAR* const* Value = PluginInfo.TryGetDataByKey<const TCHAR*>(IndexWithinRowData))
						{
							return FTableCellValue(*Value);
						}
					}

					return TOptional<FTableCellValue>();
				}

			private:
				int32 IndexWithinRowData;
			};

			TSharedRef<ITableCellValueGetter> Getter = MakeShared<FCustomColumnStringValueGetter>(CustomColumnIndex);
			Column.SetValueGetter(Getter);

			TSharedRef<ITableCellValueFormatter> Formatter = MakeShared<FCStringValueFormatterAsText>();
			Column.SetValueFormatter(Formatter);

			TSharedRef<ITableCellValueSorter> Sorter = MakeShared<FSorterByCStringValue>(ColumnRef);
			Column.SetValueSorter(Sorter);
		}


		Column.SetAggregation(ETableColumnAggregation::SameValue);

		AddColumn(ColumnRef);
	}
}

/*static*/TSet<int32> FAssetTableRow::GatherAllReachableNodes(const TArray<int32>& StartingNodes, const FAssetTable& OwningTable, const TSet<int32>& AdditionalNodesToStopAt, const TSet<const TCHAR*, TStringPointerSetKeyFuncs_DEPRECATED<const TCHAR*>>& RestrictToPlugins, TMap<int32, TArray<int32>>* OutRouteMap /*= nullptr*/)
{
	// "visit" ThisIndex to seed the exploration
	TSet<int32> VisitedIndices;
	TSet<int32> IndicesToVisit;
	for (int32 StartingNodeIndex : StartingNodes)
	{
		VisitedIndices.Add(StartingNodeIndex);
		if (OutRouteMap != nullptr)
		{
			OutRouteMap->Add(StartingNodeIndex, TArray<int32>{StartingNodeIndex});
		}
		for (int32 ChildIndex : OwningTable.GetAssetChecked(StartingNodeIndex).GetDependencies())
		{
			IndicesToVisit.Add(ChildIndex);
			if (OutRouteMap != nullptr)
			{
				TArray<int32> Route{StartingNodeIndex, ChildIndex};
				OutRouteMap->Add(ChildIndex, Route);
			}
		}
	}

	while (IndicesToVisit.Num() > 0)
	{
		TSet<int32>::TIterator Iterator = IndicesToVisit.CreateIterator();
		int32 CurrentIndex = *Iterator;
		Iterator.RemoveCurrent();

		const FAssetTableRow& Row = OwningTable.GetAssetChecked(CurrentIndex);
		if (ShouldSkipDueToPlugin(RestrictToPlugins, Row.GetPluginName()) || AdditionalNodesToStopAt.Contains(CurrentIndex))
		{
			if (OutRouteMap != nullptr)
			{
				OutRouteMap->Remove(CurrentIndex);
			}
			// Don't traverse outside this plugin
			continue;
		}

		VisitedIndices.Add(CurrentIndex);

		TArray<int32>* ParentRoute = nullptr;
		if (OutRouteMap != nullptr)
		{
			ParentRoute = OutRouteMap->Find(CurrentIndex);
		}

		for (int32 ChildIndex : OwningTable.GetAssetChecked(CurrentIndex).GetDependencies())
		{
			if (!VisitedIndices.Contains(ChildIndex))
			{
				if (OutRouteMap != nullptr && ensure(ParentRoute != nullptr))
				{
					TArray<int32> ChildRoute(*ParentRoute, 1);
					ChildRoute.Add(ChildIndex);

					OutRouteMap->Add(ChildIndex, ChildRoute);
				}
				IndicesToVisit.Add(ChildIndex);
			}
		}
	}

	return VisitedIndices;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

/*static*/ void FAssetTableRow::RefineDependencies(const TSet<int32>& PreviouslyVisitedIndices, const FAssetTable& OwningTable, const TSet<int32> RootIndices, const TSet<const TCHAR*, TStringPointerSetKeyFuncs_DEPRECATED<const TCHAR*>>& RestrictToGFPs, TSet<int32>& OutUniqueIndices, TSet<int32>& OutSharedIndices)
{
	// "visit" ThisIndex to seed the exploration
	TSet<int32> IndicesToVisit;
	for (int32 RootIndex : RootIndices)
	{
		const TArray<int32>& Dependencies = OwningTable.GetAssetChecked(RootIndex).GetDependencies();
		for (int32 DependencyIndex : Dependencies)
		{
			IndicesToVisit.Add(DependencyIndex);
		}
	}
	TSet<int32> NewlyExcludedDependencies;

	while (IndicesToVisit.Num() > 0)
	{
		TSet<int32>::TIterator Iterator = IndicesToVisit.CreateIterator();
		int32 CurrentIndex = *Iterator;
		Iterator.RemoveCurrent();

		const FAssetTableRow& Row = OwningTable.GetAssetChecked(CurrentIndex);
		if (ShouldSkipDueToPlugin(RestrictToGFPs, Row.GetPluginName()))
		{
			// Don't traverse outside this plugin
			continue;
		}

		if (OutSharedIndices.Contains(CurrentIndex))
		{
			// We already know not to traverse into this. We'll handle excluding its transitive dependencies later.
			continue;
		}

		bool ShouldIncludeInTotal = true;
		for (int32 ReferencerIndex : Row.Referencers)
		{
			if (PreviouslyVisitedIndices.Contains(ReferencerIndex) == false && !RootIndices.Contains(ReferencerIndex))
			{
				ShouldIncludeInTotal = false;
				break;
			}
		}
		if (ShouldIncludeInTotal == false)
		{
			NewlyExcludedDependencies.Add(CurrentIndex);
			continue;
		}
		OutUniqueIndices.Add(CurrentIndex);

		for (int32 ChildIndex : OwningTable.GetAssetChecked(CurrentIndex).GetDependencies())
		{
			// Don't revisit nodes we've already visited and don't re-add ThisIndex to avoid loops (and to avoid counting ourself)
			if (!OutUniqueIndices.Contains(ChildIndex) && !RootIndices.Contains(ChildIndex))
			{
				IndicesToVisit.Add(ChildIndex);
			}
		}
	}

	// We now have a set of nodes that we know are not uniquely owned
	// Anything they reference should also be removed from the set of unique dependencies
	TSet<int32> NewlyExcludedDependenciesAndTheirTransitiveDependencies = GatherAllReachableNodes(NewlyExcludedDependencies.Array(), OwningTable, RootIndices, RestrictToGFPs);
	OutSharedIndices = OutSharedIndices.Union(NewlyExcludedDependenciesAndTheirTransitiveDependencies);
	OutUniqueIndices = OutUniqueIndices.Difference(NewlyExcludedDependenciesAndTheirTransitiveDependencies);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

/*static*/ FAssetTableDependencySizes FAssetTableRow::ComputeDependencySizes(const FAssetTable & OwningTable, const TSet<int32>& RootIndices, TSet<int32>* OutUniqueDependencies, TSet<int32>* OutSharedDependencies)
{
	QUICK_SCOPE_CYCLE_COUNTER(STAT_FAssetTableRow_ComputeDependencySizes);

	FAssetTableDependencySizes Result;

	TSet<const TCHAR*, TStringPointerSetKeyFuncs_DEPRECATED<const TCHAR*>> RestrictToPlugins;
	for (int32 RootIndex : RootIndices)
	{
		RestrictToPlugins.Add(OwningTable.GetAssetChecked(RootIndex).GetPluginName());
	}


	TSet<int32> VisitedIndices = GatherAllReachableNodes(RootIndices.Array(), OwningTable, TSet<int32>(), RestrictToPlugins);

	// Iteratively separate the graph of "all things referenced by ThisIndex, directly or indirectly"
	// into "UniqueDependencies -- things referenced ONLY by ThisIndex and by other things themselves referenced ONLY by ThisIndex" limited to the GFPs of the RootIndices
	// and "SharedDependencies" -- things removed from the list of "all things referenced by ThisIndex" in order to identify UniqueDependencies, limited to the GFPs of the RootIndices
	TSet<int32> UniqueDependencies;
	TSet<int32> SharedDependencies;
	RefineDependencies(VisitedIndices, OwningTable, RootIndices, RestrictToPlugins, UniqueDependencies, SharedDependencies);

	// If there's only one root provided, the dependencies shouldn't include it. If more than one root is provided,
	// some roots might be included as dependencies of other roots
	ensure(RootIndices.Num() > 1 || UniqueDependencies.Intersect(RootIndices).IsEmpty());

	// if there is more than one root index, remove any root indices from the dependency lists in order to ensure they aren't double counted
	// (they should be accounted for as part of the self sizes of the selected assets)
	UniqueDependencies = UniqueDependencies.Difference(RootIndices);
	SharedDependencies = SharedDependencies.Difference(RootIndices);

	for (int32 Index : UniqueDependencies)
	{
		Result.UniqueDependenciesSize += OwningTable.GetAssetChecked(Index).GetStagedCompressedSizeRequiredInstall();
	}
	if (OutUniqueDependencies != nullptr)
	{
		*OutUniqueDependencies = UniqueDependencies;
	}

	// Now explore all the dependencies of SharedDependencies and gather up their sizes
	// This is necessary because the process calling RefineDependencies doesn't produce a complete list of other dependencies,
	// just a partial set. By exploring the dependencies of that partial set we can find all the things that were
	// referenced by ThisIndex but also by some other asset outside the subgraph defined by ThisIndex and its UniqueDependencies
	VisitedIndices.Empty();

	while (SharedDependencies.Num() > 0)
	{
		TSet<int32>::TIterator Iterator = SharedDependencies.CreateIterator();
		int32 CurrentIndex = *Iterator;
		Iterator.RemoveCurrent();

		const FAssetTableRow& Row = OwningTable.GetAssetChecked(CurrentIndex);
		VisitedIndices.Add(CurrentIndex);
		if (ShouldSkipDueToPlugin(RestrictToPlugins, Row.GetPluginName()))
		{
			// Don't traverse outside this plugin
			continue;
		}

		int64 DependencySize = OwningTable.GetAssetChecked(CurrentIndex).GetStagedCompressedSizeRequiredInstall();
		Result.SharedDependenciesSize += DependencySize;
		if (OutSharedDependencies != nullptr)
		{
			OutSharedDependencies->Add(CurrentIndex);
		}

		for (int32 ChildIndex : OwningTable.GetAssetChecked(CurrentIndex).GetDependencies())
		{
			if (!VisitedIndices.Contains(ChildIndex) && !RootIndices.Contains(ChildIndex))
			{
				SharedDependencies.Add(ChildIndex);
			}
		}
	}

	return Result;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

/*static*/ int64 FAssetTableRow::ComputeTotalSizeExternalDependencies(const FAssetTable& OwningTable, const TSet<int32>& StartingNodes, TSet<int32>* OutExternalDependencies/* = nullptr*/, TMap<int32, TArray<int32>>* OutRouteMap/* = nullptr*/)
{
	UE::Insights::FStopwatch Stopwatch;
	Stopwatch.Start();

	TSet<int32> AllReachableNodes = GatherAllReachableNodes(StartingNodes.Array(), OwningTable, TSet<int32>{}, TSet<const TCHAR*, TStringPointerSetKeyFuncs_DEPRECATED<const TCHAR*>>{}, OutRouteMap);
	TSet<const TCHAR*, TStringPointerSetKeyFuncs_DEPRECATED<const TCHAR*>> SourcePlugins;
	for (int32 RootIndex : StartingNodes)
	{
		SourcePlugins.Add(OwningTable.GetAsset(RootIndex)->GetPluginName());
	}

	int64 TotalSizeExternalDependencies = 0;
	int32 NumDependenciesIncluded = 0;
	for (int32 Index : AllReachableNodes)
	{
		const FAssetTableRow& Row = *OwningTable.GetAsset(Index);
		// Only include EXTERNAL dependencies
		if (!SourcePlugins.Contains(Row.PluginName))
		{
			if (OutExternalDependencies != nullptr)
			{
				OutExternalDependencies->Add(Index);
			}
			TotalSizeExternalDependencies += Row.StagedCompressedSizeRequiredInstall;
			NumDependenciesIncluded++;
		}
	}

	Stopwatch.Stop();
	const double Duration = Stopwatch.GetAccumulatedTime();
	if (Duration > 1.0)
	{
		if (StartingNodes.Num() == 1)
		{
			int32 RowIndex = *StartingNodes.CreateConstIterator();
			const FAssetTableRow& Row = *OwningTable.GetAsset(RowIndex);
			UE_LOGF(LogInsights, Log, "ComputeTotalSizeExternalDependencies(%ls%ls) : %.3fs", Row.Path, Row.Name, Duration);
		}
		else
		{
			UE_LOGF(LogInsights, Log, "ComputeTotalSizeExternalDependencies(<%d Assets>) --> %d Dependencies : %.3fs", StartingNodes.Num(), NumDependenciesIncluded, Duration);
		}
	}

	return TotalSizeExternalDependencies;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int64 FAssetTableRow::GetOrComputeTotalSizeUniqueDependencies(const FAssetTable& OwningTable, int32 ThisIndex) const
{
	if (TotalSizeUniqueDependencies == -1)
	{
		UE::Insights::FStopwatch Stopwatch;
		Stopwatch.Start();

		FAssetTableDependencySizes Sizes = ComputeDependencySizes(OwningTable, TSet<int32>{ThisIndex});
		TotalSizeUniqueDependencies = Sizes.UniqueDependenciesSize;
		TotalSizeSharedDependencies = Sizes.SharedDependenciesSize;

		Stopwatch.Stop();
		const double Duration = Stopwatch.GetAccumulatedTime();
		if (Duration > 0.2f)
		{
			UE_LOGF(LogInsights, Log, "ComputeDependencySizes(%ls%ls) : %.3fs", Path, Name, Duration);
		}
	}
	return TotalSizeUniqueDependencies;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int64 FAssetTableRow::GetOrComputeTotalSizeSharedDependencies(const FAssetTable& OwningTable, int32 ThisIndex) const
{
	if (TotalSizeSharedDependencies == -1)
	{
		UE::Insights::FStopwatch Stopwatch;
		Stopwatch.Start();

		FAssetTableDependencySizes Sizes = ComputeDependencySizes(OwningTable, TSet<int32>{ThisIndex});
		TotalSizeUniqueDependencies = Sizes.UniqueDependenciesSize;
		TotalSizeSharedDependencies = Sizes.SharedDependenciesSize;

		Stopwatch.Stop();
		const double Duration = Stopwatch.GetAccumulatedTime();
		if (Duration > 1.0)
		{
			UE_LOGF(LogInsights, Log, "ComputeDependencySizes(%ls%ls) : %.3fs", Path, Name, Duration);
		}
	}
	return TotalSizeSharedDependencies;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int64 FAssetTableRow::GetOrComputeTotalSizeExternalDependencies(const FAssetTable& OwningTable, int32 ThisIndex) const
{
	if (TotalSizeExternalDependencies == -1)
	{
		TotalSizeExternalDependencies = ComputeTotalSizeExternalDependencies(OwningTable, TSet<int32>{ThisIndex});
	}
	return TotalSizeExternalDependencies;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int64 FAssetTablePluginInfo::GetOrComputeTotalSizeInclusiveOfDependencies(const FAssetTable& OwningTable) const
{
	if (InclusiveSize == -1)
	{
		InclusiveSize = Size;

		TArray<int32> DependencyStack = PluginDependencies;
		TSet<int32> AllDependencies;

		// Gather all the dependencies
		while (DependencyStack.Num() > 0)
		{
			int32 CurrentIndex = DependencyStack.Pop();
			if (AllDependencies.Contains(CurrentIndex))
			{
				continue;
			}
			AllDependencies.Add(CurrentIndex);
			const FAssetTablePluginInfo& PluginInfo = OwningTable.GetPluginInfoByIndex(CurrentIndex);
			InclusiveSize += PluginInfo.GetSize();
			DependencyStack.Append(PluginInfo.GetDependencies());
		}
	}
	return InclusiveSize;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int64 FAssetTablePluginInfo::GetOrComputeTotalSizeUniqueDependencies(const FAssetTable& OwningTable) const
{
	if (UniqueDependenciesSize == -1)
	{
		ComputeDependencySizes(OwningTable);
	}
	return UniqueDependenciesSize;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

int64 FAssetTablePluginInfo::GetOrComputeTotalSizeSharedDependencies(const FAssetTable& OwningTable) const
{
	if (SharedDependenciesSize == -1)
	{
		ComputeDependencySizes(OwningTable);
	}
	return SharedDependenciesSize;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetTablePluginInfo::ComputeTotalSelfAndInclusiveSizes(const FAssetTable& OwningTable, const TSet<int32>& RootPlugins, int64& OutTotalSelfSize, int64& OutTotalInclusiveSize)
{
	TSet<int32> VisitedNodes;
	TArray<int32> DependencyStack = RootPlugins.Array();
	OutTotalSelfSize = 0;
	OutTotalInclusiveSize = 0;

	while (DependencyStack.Num() > 0)
	{
		int32 CurrentIndex = DependencyStack.Pop();

		if (VisitedNodes.Contains(CurrentIndex))
		{
			continue;
		}
		VisitedNodes.Add(CurrentIndex);
		
		const FAssetTablePluginInfo& PluginInfo = OwningTable.GetPluginInfoByIndexChecked(CurrentIndex);
		if (RootPlugins.Contains(CurrentIndex))
		{
			OutTotalSelfSize += PluginInfo.GetSize();
		}
		OutTotalInclusiveSize += PluginInfo.GetSize();
		DependencyStack.Append(PluginInfo.GetDependencies());
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FAssetTablePluginInfo::ComputeDependencySizes(const FAssetTable& OwningTable) const
{
	SharedDependenciesSize = 0;
	UniqueDependenciesSize = 0;

	// Plugins can't have circular dependencies, so our algorithm for exploring the dependencies is significantly simpler than for assets

	const int32 RootIndex = OwningTable.GetIndexForPlugin(PluginName);

	TArray<int32> DependencyStack = PluginDependencies;
	TSet<int32> AllDependencies;
	bool bAlreadyExists = false;
	// Gather all the dependencies
	while (DependencyStack.Num() > 0)
	{
		int32 CurrentIndex = DependencyStack.Pop();
		AllDependencies.Add(CurrentIndex, &bAlreadyExists);
		if(!bAlreadyExists)
		{
			DependencyStack.Append(OwningTable.GetPluginInfoByIndex(CurrentIndex).GetDependencies());
		}
	}

	// Find the set of dependencies which are directly referenced by something outside this set of plugins
	TArray<int32> SharedDependencyStack;
	for (int32 DependencyIndex : AllDependencies)
	{
		const FAssetTablePluginInfo& CurrentPluginInfo = OwningTable.GetPluginInfoByIndex(DependencyIndex);
		bool IsShared = false;
		for (int32 ReferencerIndex : CurrentPluginInfo.GetReferencers())
		{
			bool ReferencerIsRoot = (ReferencerIndex == RootIndex);
			bool ReferencerIsNotInDependencyTree = !AllDependencies.Contains(ReferencerIndex);
			if (ReferencerIsNotInDependencyTree && !ReferencerIsRoot)
			{
				IsShared = true;
				break;
			}
		}
		if (IsShared)
		{
			SharedDependencyStack.Add(DependencyIndex);
		}
	}
	
	// Add to the SharedDependencies list all transitive dependencies
	TSet<int32> SharedDependencies;
	while (SharedDependencyStack.Num() > 0)
	{
		int32 CurrentIndex = SharedDependencyStack.Pop();
		SharedDependencies.Add(CurrentIndex, &bAlreadyExists);
		if (!bAlreadyExists)
		{
			SharedDependencyStack.Append(OwningTable.GetPluginInfoByIndex(CurrentIndex).GetDependencies());
		}
	}

	// UniqueDependencies are all the dependencies that aren't shared
	TArray<int32> UniqueDependencies = AllDependencies.Array();
	for (int32 SharedDependencyIndex : SharedDependencies)
	{
		UniqueDependencies.RemoveSwap(SharedDependencyIndex);
	}
	
	ensure((UniqueDependencies.Num() + SharedDependencies.Num()) == AllDependencies.Num());

	// Add up the sizes
	for (int32 UniqueDependencyIndex : UniqueDependencies)
	{
		const FAssetTablePluginInfo& CurrentPluginInfo = OwningTable.GetPluginInfoByIndexChecked(UniqueDependencyIndex);
		int64 PluginSize = CurrentPluginInfo.GetSize();
		ensureMsgf(PluginSize >= 0, TEXT("Found plugin %s with uninitialized size."), CurrentPluginInfo.GetName());
		UniqueDependenciesSize += PluginSize;
	}

	for (int32 SharedDependencyIndex : SharedDependencies)
	{
		const FAssetTablePluginInfo& CurrentPluginInfo = OwningTable.GetPluginInfoByIndexChecked(SharedDependencyIndex);
		int64 PluginSize = CurrentPluginInfo.GetSize();
		ensureMsgf(PluginSize >= 0, TEXT("Found plugin %s with uninitialized size."), CurrentPluginInfo.GetName());
		SharedDependenciesSize += PluginSize;
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
