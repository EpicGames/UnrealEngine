// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AssetRegistry/AssetData.h"
#include "Containers/StringView.h"
#include "InsightsCore/Table/ViewModels/Table.h"
#include "Misc/CString.h"
#include "Misc/TVariant.h"

namespace UE
{
	namespace Insights
	{
		class FTableColumn;
	}
}

class FAssetTable;

// Simple string store.
class FAssetTableStringStore
{
private:
	static constexpr int32 ChunkBufferLen = 1024 * 1024; // 2 MiB/buffer
	static constexpr ESearchCase::Type SearchCase = ESearchCase::CaseSensitive;

	struct FChunk
	{
		TCHAR* Buffer = nullptr;
		uint32 Used = 0;
	};

public:
	FAssetTableStringStore();
	~FAssetTableStringStore();

	// Resets this string store. Frees all memory.
	void Reset();

	// Store a string. Gets the stored string for specified input string.
	// @param InStr - the input string
	// @returns the stored string view.
	const FStringView Store(const TCHAR* InStr);

	// Store a string. Gets the stored string for specified input string.
	// The returned string is guaranteed to be null terminated.
	// @param InStr - the input string
	// @returns the stored string view.
	const FStringView Store(const FStringView InStr);

	// Gets the maximum length of a single string that can be stored.
	// @returns the maximum string length
	int32 GetMaxStringLength() const
	{
		return ChunkBufferLen - 1;
	}

	// Gets number of input strings, including duplicates.
	// @returns number of input strings.
	int32 GetNumInputStrings() const
	{
		return NumInputStrings;
	}

	// Gets size of input strings, including duplicates, in [bytes].
	// @returns size of input strings, in [bytes].
	SIZE_T GetTotalInputStringSize() const
	{
		return TotalInputStringSize;
	}

	// Gets number of stored (unique) strings.
	// @returns number of stored strings.
	int32 GetNumStrings() const
	{
		return NumStoredStrings;
	}

	// Gets size of stored (unique) strings, in [bytes].
	// @returns size of stored strings, in [bytes].
	SIZE_T GetTotalStringSize() const
	{
		return TotalStoredStringSize;
	}

	// Gets (estimated) total allocated memory for this string store, in [bytes].
	// @returns memory size, in [bytes].
	SIZE_T GetAllocatedSize() const
	{
		return Chunks.Num() * ChunkBufferLen * sizeof(TCHAR)
			+ Chunks.GetAllocatedSize()
			+ Cache.GetAllocatedSize();
	}

	void EnumerateStrings(TFunction<void(const FStringView Str)> Callback) const;

private:
	void AddChunk();

private:
	TArray<FChunk> Chunks;
	TMultiMap<uint32, FStringView> Cache; // Hash --> FStringView
	SIZE_T TotalInputStringSize;
	SIZE_T TotalStoredStringSize;
	int32 NumInputStrings;
	int32 NumStoredStrings;
};

// Column identifiers
struct FAssetTableColumns
{
	static const FName CountColumnId;
	static const FName TypeColumnId;
	static const FName NameColumnId;
	static const FName PathColumnId;
	static const FName PrimaryTypeColumnId;
	static const FName PrimaryNameColumnId;
	static const FName StagedCompressedSizeRequiredInstallColumnId;
	static const FName TotalSizeUniqueDependenciesColumnId;
	static const FName TotalSizeSharedDependenciesColumnId;
	static const FName TotalSizeExternalDependenciesColumnId;
	static const FName TotalUsageCountColumnId;
	static const FName ChunksColumnId;
	static const FName NativeClassColumnId;
	static const FName PluginNameColumnId;
	static const FName PluginInclusiveSizeColumnId;
	static const FName PluginTypeColumnId;
};

struct FAssetTableDependencySizes
{
	int64 UniqueDependenciesSize = 0;
	int64 SharedDependenciesSize = 0;
};

class FAssetTablePluginInfo
{
	friend class FAssetTable;
	friend class SAssetTableTreeView;

public:
	const TCHAR* GetName() const { return PluginName; }

	int32 GetNumDependencies() const { return PluginDependencies.Num(); }
	const TArray<int32>& GetDependencies() const { return PluginDependencies; }
	int32 GetNumReferencers() const { return PluginReferencers.Num(); }
	const TArray<int32>& GetReferencers() const { return PluginReferencers; }
	int64 GetSize() const { return Size; }
	bool IsRootPlugin() const { return bIsRootPlugin; }
	const TCHAR* GetPluginType() const { return PluginTypeString; }

	int64 GetOrComputeTotalSizeInclusiveOfDependencies(const FAssetTable& OwningTable) const;

	// This is the total size of all dependencies which are not used by any plugin that is not this plugin or one of its other dependencies
	int64 GetOrComputeTotalSizeUniqueDependencies(const FAssetTable& OwningTable) const;
	// This is the total size of all dependencies which are used by some plugin not referenced (directly or indirectly) by this plugin
	int64 GetOrComputeTotalSizeSharedDependencies(const FAssetTable& OwningTable) const;

	static void ComputeTotalSelfAndInclusiveSizes(const FAssetTable& OwningTable, const TSet<int32>& RootPlugins, int64& OutTotalSelfSize, int64& OutTotalInclusiveSize);

	TSet<int32> GetPluginAssets() const { return PluginAssets; };

	void AddPluginAsset(const int32 RowIndex) { PluginAssets.Add(RowIndex); };

	template<typename T> const T* TryGetDataByKey(int32 Key) const
	{
		// In general there are expected to be very few keys since each key corresponds to a custom implemented
		// column. Therefore a simple linear search is expected to be sufficiently performant.
		for (const FCustomColumnData& CustomDataEntry : CustomColumnData)
		{
			if (CustomDataEntry.Key > Key)
			{
				// We've passed where this key ought to be found. No entry is present.
				break; 
			}
			else if (CustomDataEntry.Key == Key)
			{
				return CustomDataEntry.Value.TryGet<T>();
			}
		}
		return nullptr;
	}

private:
	void ComputeDependencySizes(const FAssetTable& OwningTable) const;

	TArray<int32> PluginDependencies;
	TArray<int32> PluginReferencers;
	TSet<int32> PluginAssets;

	typedef TVariant<bool, const TCHAR*> CustomFieldVariantType;

	struct FCustomColumnData
	{
		CustomFieldVariantType Value;
		int32 Key;
	};

	TArray<FCustomColumnData> CustomColumnData;

	const TCHAR* PluginName = nullptr;
	const TCHAR* PluginTypeString = nullptr;
	int64 Size = 0;
	mutable int64 InclusiveSize = -1;
	mutable int64 UniqueDependenciesSize = -1;
	mutable int64 SharedDependenciesSize = -1;
	bool bIsRootPlugin = false;
};

class FAssetTableRow
{
	friend class FAssetTable;
	friend class SAssetTableTreeView;

public:
	FAssetTableRow()
	{
	}
	~FAssetTableRow() = default;

	FName GetNodeName() const { return FName(Name, 0); }

	const TCHAR* GetType() const { return Type; }
	const TCHAR* GetName() const { return Name; }
	const TCHAR* GetPath() const { return Path; }
	const TCHAR* GetPrimaryType() const { return PrimaryType; }
	const TCHAR* GetPrimaryName() const { return PrimaryName; }
	int64 GetStagedCompressedSizeRequiredInstall() const { return StagedCompressedSizeRequiredInstall; }
	int64 GetOrComputeTotalSizeUniqueDependencies(const FAssetTable& OwningTable, int32 ThisIndex) const;
	int64 GetOrComputeTotalSizeSharedDependencies(const FAssetTable& OwningTable, int32 ThisIndex) const; 
	int64 GetOrComputeTotalSizeExternalDependencies(const FAssetTable& OwningTable, int32 ThisIndex) const; 
	int64 GetTotalUsageCount() const { return TotalUsageCount; }
	const TCHAR* GetChunks() const { return Chunks; }
	const TCHAR* GetNativeClass() const { return NativeClass; }
	const TCHAR* GetPluginName() const { return PluginName; }
	int32 GetNumDependencies() const { return Dependencies.Num(); }
	const TArray<int32>& GetDependencies() const { return Dependencies; }
	int32 GetNumReferencers() const { return Referencers.Num(); }
	const TArray<int32>& GetReferencers() const { return Referencers; }
	FLinearColor GetColor() const { return Color; }
	const FSoftObjectPath& GetSoftObjectPath() const { return SoftObjectPath; }

	static FAssetTableDependencySizes ComputeDependencySizes(const FAssetTable& OwningTable, const TSet<int32>& RootIndices, TSet<int32>* OutUniqueDependencies = nullptr, TSet<int32>* OutSharedDependencies = nullptr);

	static int64 ComputeTotalSizeExternalDependencies(const FAssetTable& OwningTable, const TSet<int32>& StartingNodes, TSet<int32>* OutExternalDependencies = nullptr, TMap<int32, TArray<int32>>* OutRouteMap = nullptr);

	// Finds all nodes reachable from StartingNodes INCLUDING those in StartingNodes
	// An optional OutRouteMap may be provided. For each discovered reachable node, the routemap will get a value entry containing an array of nodes that constitute one possible path from a StartingNode to the destination
	static TSet<int32> GatherAllReachableNodes(const TArray<int32>& StartingNodes, const FAssetTable& OwningTable, const TSet<int32>& AdditionalNodesToStopAt, const TSet<const TCHAR*, TStringPointerSetKeyFuncs_DEPRECATED<const TCHAR*>>& RestrictToPlugins, TMap<int32, TArray<int32>>* OutRouteMap = nullptr);

private:
	static void RefineDependencies(const TSet<int32>& PreviouslyVisitedIndices, const FAssetTable& OwningTable, const TSet<int32> RootIndices, const TSet<const TCHAR*, TStringPointerSetKeyFuncs_DEPRECATED<const TCHAR*>>& RestrictToGFPs, TSet<int32>& OutUniqueIndices, TSet<int32>& OutSharedIndices);
	static inline bool ShouldSkipDueToPlugin(const TSet<const TCHAR*, TStringPointerSetKeyFuncs_DEPRECATED<const TCHAR*>>& RestrictToPlugins, const TCHAR* PluginName) { return (RestrictToPlugins.Num() > 0) && !RestrictToPlugins.Contains(PluginName); }


private:
	FSoftObjectPath SoftObjectPath; // Used to allow you to go to that asset in the content browser or to open its editor after lookingup the FAssetData in the original registry state
	const TCHAR* Type = nullptr;
	const TCHAR* Name = nullptr;
	const TCHAR* Path = nullptr;
	const TCHAR* PrimaryType = nullptr;
	const TCHAR* PrimaryName = nullptr;
	int64 StagedCompressedSizeRequiredInstall = 0;
	mutable int64 TotalSizeUniqueDependencies = -1; // Lazily calculated
	mutable int64 TotalSizeSharedDependencies = -1; // Lazily calculated
	mutable int64 TotalSizeExternalDependencies = -1; // Lazily calculated
	int64 TotalUsageCount = 0;
	const TCHAR* Chunks = nullptr;
	const TCHAR* NativeClass = nullptr;
	const TCHAR* PluginName = nullptr;
	TArray<int32> Dependencies;
	TArray<int32> Referencers;
	FLinearColor Color;
};

class FAssetTable : public UE::Insights::FTable
{
public:
	FAssetTable();
	virtual ~FAssetTable();

	virtual void Reset();

	TArray<FAssetTableRow>& GetAssets() { return Assets; }
	const TArray<FAssetTableRow>& GetAssets() const { return Assets; }

	bool IsValidRowIndex(int32 InIndex) const { return InIndex >= 0 && InIndex < Assets.Num(); }
	FAssetTableRow* GetAsset(int32 InIndex) { return IsValidRowIndex(InIndex) ? &Assets[InIndex] : nullptr; }
	const FAssetTableRow* GetAsset(int32 InIndex) const { return IsValidRowIndex(InIndex) ? &Assets[InIndex] : nullptr; }
	FAssetTableRow& GetAssetChecked(int32 InIndex) { check(IsValidRowIndex(InIndex)); return Assets[InIndex]; }
	const FAssetTableRow& GetAssetChecked(int32 InIndex) const { check(IsValidRowIndex(InIndex)); return Assets[InIndex]; }

	int32 GetTotalAssetCount() const { return Assets.Num(); }
	int32 GetVisibleAssetCount() const { return VisibleAssetCount; }
	int32 GetHiddenAssetCount() const { return Assets.Num() - VisibleAssetCount; }

	void SetVisibleAssetCount(int32 InVisibleAssetCount) { check(VisibleAssetCount <= Assets.Num()); VisibleAssetCount = InVisibleAssetCount; }

	void AddAsset(const FAssetTableRow& AssetRow) { Assets.Add(AssetRow); }

	FAssetTablePluginInfo& GetOrCreatePluginInfo(const TCHAR* StoredPluginName);

	const FAssetTablePluginInfo& GetPluginInfoByIndex(int32 PluginIndex) const { return Plugins[PluginIndex]; }
	FAssetTablePluginInfo& GetPluginInfoByIndex(int32 PluginIndex) { return Plugins[PluginIndex]; }
	const FAssetTablePluginInfo& GetPluginInfoByIndexChecked(int32 PluginIndex) const { check(IsValidPluginIndex(PluginIndex)); return Plugins[PluginIndex]; }
	int32 GetIndexForPlugin(const TCHAR* StoredPluginName) const;
	int32 GetNumPlugins() const { return Plugins.Num(); }

	FName GetNameForPlugin(int32 InPluginIndex) const
	{
		if (IsValidPluginIndex(InPluginIndex))
		{
			return FName(Plugins[InPluginIndex].PluginName);
		}
		return FName("<InvalidPlugin>");
	}

	bool IsValidPluginIndex(int32 Index) const { return Index >= 0 && Index < Plugins.Num(); }

	void EnumeratePluginDependencies(const FAssetTablePluginInfo& InPlugin, TFunction<void(int32 PluginIndex)> Callback) const;
	void EnumerateAssetsForPlugin(const FAssetTablePluginInfo& InPlugin, TFunction<void(int32 AssetIndex)> Callback) const;

	const FStringView StoreString(const FStringView InStr) { return StringStore.Store(InStr); }
	const TCHAR* StoreStr(const FString& InStr) { return StringStore.Store(InStr).GetData(); }
	const FAssetTableStringStore& GetStringStore() const { return StringStore; }
	FAssetTableStringStore& GetStringStore() { return StringStore; }

	void ClearAllData()
	{
		Assets.Empty();
		VisibleAssetCount = 0;
		StringStore.Reset();
		Plugins.Empty();
		PluginNameToIndexMap.Empty();
		CustomColumns.Empty();
	}

	void PopulateAssetsInPlugins();

	enum class ECustomColumnDefinitionType
	{
		Boolean,
		String
	};

	struct FCustomColumnDefinition
	{
		FName ColumnId;
		ECustomColumnDefinitionType Type;
		int32 Key;
	};

	void AddCustomColumn(const FCustomColumnDefinition& ColumnDefinition) { CustomColumns.Add(ColumnDefinition); }

private:
	void AddDefaultColumns();

private:
	TArray<FAssetTableRow> Assets;
	TArray<FAssetTablePluginInfo> Plugins;
	TMap<const TCHAR*, int32, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<const TCHAR*, int32>> PluginNameToIndexMap;
	TArray<FCustomColumnDefinition> CustomColumns;
	int32 VisibleAssetCount = 0;
	FAssetTableStringStore StringStore;
};
