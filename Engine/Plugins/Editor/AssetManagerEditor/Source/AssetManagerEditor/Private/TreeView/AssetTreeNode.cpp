// Copyright Epic Games, Inc. All Rights Reserved.

#include "AssetTreeNode.h"

#include "AssetDependencyGrouping.h"
#include "InsightsCore/Common/AsyncOperationProgress.h"
#include "InsightsCore/Common/InsightsCoreStyle.h"
#include "InsightsCore/Table/Widgets/STableTreeView.h"
#include "Internationalization/Internationalization.h"
#include "Styling/StyleColors.h"

#define LOCTEXT_NAMESPACE "FAssetTreeNode"

INSIGHTS_IMPLEMENT_RTTI(FAssetTreeNode)
INSIGHTS_IMPLEMENT_RTTI(FAssetDependenciesGroupTreeNode)
INSIGHTS_IMPLEMENT_RTTI(FPluginSimpleGroupNode)
INSIGHTS_IMPLEMENT_RTTI(FPluginAndDependenciesGroupNode)
INSIGHTS_IMPLEMENT_RTTI(FPluginDependenciesGroupNode)

////////////////////////////////////////////////////////////////////////////////////////////////////
// FAssetTreeNode
////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* FAssetTreeNode::GetIcon(EStyle Style) const
{
	switch ( Style )
	{
		default:
		case EStyle::EDefault:
		{
			return UE::Insights::FInsightsCoreStyle::GetBrush("Icons.Leaf.TreeItem");
		}

		case EStyle::EAsset:
		{
			return UE::Insights::FInsightsCoreStyle::GetBrush("Icons.Asset.TreeItem");
		}
				
		case EStyle::EGroup:
		{
			return UE::Insights::FInsightsCoreStyle::GetBrush("Icons.Group.TreeItem");
		}	

		case EStyle::EDependencies:
		{
			return UE::Insights::FInsightsCoreStyle::GetBrush("Icons.Dependencies.TreeItem");
		}

		case EStyle::EPlugin:
		{
			return UE::Insights::FInsightsCoreStyle::GetBrush("Icons.Plugin.TreeItem");
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FAssetTreeNode::GetIconColor(EStyle Style) const
{
	switch (Style)
	{
		default:
		case EStyle::EDefault:
		{
			return USlateThemeManager::Get().GetColor(EStyleColor::AccentWhite);
		}

		case EStyle::EAsset:
		{
			return GetAssetChecked().GetColor();
		}

		case EStyle::EGroup:
		{
			return USlateThemeManager::Get().GetColor(EStyleColor::AccentYellow);
		}

		case EStyle::EDependencies:
		{
			return USlateThemeManager::Get().GetColor(EStyleColor::AccentWhite);
		}

		case EStyle::EPlugin:
		{
			return USlateThemeManager::Get().GetColor(EStyleColor::AccentGreen);
		}
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FAssetTreeNode::GetColor(EStyle Style) const 
{
	return USlateThemeManager::Get().GetColor(EStyleColor::AccentWhite);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetTreeNode::EStyle FAssetTreeNode::GetStyle() const
{
	return IsGroup() ? EStyle::EGroup : (IsValidAsset() ? EStyle::EAsset : EStyle::EDefault);
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FSlateBrush* FAssetTreeNode::GetIcon() const
{
	return GetIcon(GetStyle());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FAssetTreeNode::GetIconColor() const
{
	return GetIconColor(GetStyle());
}

////////////////////////////////////////////////////////////////////////////////////////////////////

FLinearColor FAssetTreeNode::GetColor() const
{
	return GetColor(GetStyle());
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FAssetDependenciesGroupTreeNode
////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetTreeNode::EStyle FAssetDependenciesGroupTreeNode::GetStyle() const
{
	return EStyle::EDependencies;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FAssetDependenciesGroupTreeNode::GetExtraDisplayName() const
{
	if (!bAreChildrenCreated)
	{
		return LOCTEXT("DblClickToExpand", "(double click to expand)");
	}
	return FTableTreeNode::GetExtraDisplayName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FAssetDependenciesGroupTreeNode::OnLazyCreateChildren(TSharedPtr<UE::Insights::STableTreeView> InTableTreeView)
{
	if (bAreChildrenCreated)
	{
		return false;
	}

	TSharedPtr<STreeView<UE::Insights::FTableTreeNodePtr>> TreeView = InTableTreeView->GetInnerTreeView();
	if (!TreeView || !TreeView->IsItemExpanded(SharedThis(this)))
	{
		return false;
	}

	FAssetTable& AssetTable = GetAssetTableChecked();
	const FAssetTableRow& AssetRow = AssetTable.GetAssetChecked(GetRowIndex());
	const TArray<int32>& Dependencies = AssetRow.GetDependencies();
	TArray<UE::Insights::FTableTreeNodePtr> AddedNodes;
	for (int32 DepAssetIndex : Dependencies)
	{
		if (!ensure(AssetTable.IsValidRowIndex(DepAssetIndex)))
		{
			continue;
		}
		const FAssetTableRow& DepAssetRow = AssetTable.GetAssetChecked(DepAssetIndex);
		FName DepAssetNodeName(DepAssetRow.GetName());
		FAssetTreeNodePtr DepAssetNodePtr = MakeShared<FAssetTreeNode>(DepAssetNodeName, GetAssetTableWeak(), DepAssetIndex);
		AddedNodes.Add(DepAssetNodePtr);
	}

	UE::Insights::FAsyncOperationProgress Progress;
	FAssetDependencyGrouping Grouping;
	Grouping.GroupNodes(AddedNodes, *this, GetParentTable(), Progress);

	bAreChildrenCreated = true;
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FPluginSimpleGroupNode
////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetTreeNode::EStyle FPluginSimpleGroupNode::GetStyle() const
{
	return EStyle::EGroup;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

void FPluginSimpleGroupNode::AddAssetChildrenNodes()
{
	// [this]
	// |
	// +-- [asset:{Asset1}]
	// |
	// +-- [asset:{Asset2}]
	// ...

	FAssetTable& AssetTable = GetAssetTableChecked();
	if (AssetTable.IsValidPluginIndex(PluginIndex))
	{
		const FAssetTablePluginInfo& PluginInfo = AssetTable.GetPluginInfoByIndex(PluginIndex);
		AssetTable.EnumerateAssetsForPlugin(PluginInfo, [this, AssetTablePtr = &AssetTable](int32 AssetIndex)
			{
				FName AssetNodeName(AssetTablePtr->GetAssetChecked(AssetIndex).GetName());
				FAssetTreeNodePtr AssetNode = MakeShared<FAssetTreeNode>(AssetNodeName, GetAssetTableWeak(), AssetIndex);
				AddChildAndSetParent(AssetNode);
			});
	}
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FPluginAndDependenciesGroupNode
////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetTreeNode::EStyle FPluginAndDependenciesGroupNode::GetStyle() const
{
	return EStyle::EPlugin;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

TSharedPtr<FPluginSimpleGroupNode> FPluginAndDependenciesGroupNode::CreateChildren()
{
	// [this]
	// |
	// +-- [group:Assets] // FPluginSimpleGroupNode
	// |
	// +-- [group:Dependencies] (double click to expand) // FPluginDependenciesGroupNode, lazy

	FAssetTable& AssetTable = GetAssetTableChecked();
	if (AssetTable.IsValidPluginIndex(PluginIndex))
	{
		// Create the Plugin Self group node (where asset nodes will be added).
		//FName PluginGroupName = AssetTable.GetNameForPlugin(PluginIndex);
		static FName PluginGroupName(TEXT("Assets"));
		TSharedPtr<FPluginSimpleGroupNode> PluginGroup = MakeShared<FPluginSimpleGroupNode>(PluginGroupName, GetAssetTableWeak(), PluginIndex);
		PluginGroup->SetAuthorGrouping(GetAuthorGrouping());
		AddChildAndSetParent(PluginGroup);

		// Create the Plugin Dependencies group node.
		// The children nodes (list of dependent plugins) will be lazy created.
		static FName DependenciesGroupName(TEXT("Dependencies"));
		TSharedPtr<FPluginDependenciesGroupNode> DependenciesGroup = MakeShared<FPluginDependenciesGroupNode>(DependenciesGroupName, GetAssetTableWeak(), PluginIndex);
		DependenciesGroup->SetAuthorGrouping(GetAuthorGrouping());
		AddChildAndSetParent(DependenciesGroup);

		return PluginGroup; // the "Assets" group node
	}
	return SharedThis(this);
}

////////////////////////////////////////////////////////////////////////////////////////////////////
// FPluginDependenciesGroupNode
////////////////////////////////////////////////////////////////////////////////////////////////////

FAssetTreeNode::EStyle FPluginDependenciesGroupNode::GetStyle() const
{
	return EStyle::EDependencies;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

const FText FPluginDependenciesGroupNode::GetExtraDisplayName() const
{
	if (!bAreChildrenCreated)
	{
		return LOCTEXT("DblClickToExpand", "(double click to expand)");
	}
	return FTableTreeNode::GetExtraDisplayName();
}

////////////////////////////////////////////////////////////////////////////////////////////////////

bool FPluginDependenciesGroupNode::OnLazyCreateChildren(TSharedPtr<class UE::Insights::STableTreeView> InTableTreeView)
{
	if (bAreChildrenCreated)
	{
		return false;
	}

	TSharedPtr<STreeView<UE::Insights::FTableTreeNodePtr>> TreeView = InTableTreeView->GetInnerTreeView();
	if (!TreeView || !TreeView->IsItemExpanded(SharedThis(this)))
	{
		return false;
	}

	// [this]
	// |
	// +-- [group:{DependentPlugin1}] // FPluginAndDependenciesGroupNode
	// |   |
	// |   +-- [group:Assets] // FPluginSimpleGroupNode
	// |   +-- [group:Dependencies] (double click to expand) // FPluginDependenciesGroupNode, lazy
	// |
	// +-- [group:{DependentPlugin2}] // FPluginAndDependenciesGroupNode
	// |   |
	// |   +-- [group:Assets] // FPluginSimpleGroupNode
	// |   +-- [group:Dependencies] (double click to expand) // FPluginDependenciesGroupNode, lazy
	// |
	// |   ...
	// |

	FAssetTable& AssetTable = GetAssetTableChecked();
	if (AssetTable.IsValidPluginIndex(PluginIndex))
	{
		const FAssetTablePluginInfo& PluginInfo = AssetTable.GetPluginInfoByIndex(PluginIndex);

		// Add dependent plugins.
		for (int32 DependentPluginIndex : PluginInfo.GetDependencies())
		{
			if (AssetTable.IsValidPluginIndex(DependentPluginIndex))
			{
				FName PluginGroupName = AssetTable.GetNameForPlugin(DependentPluginIndex);
				const FAssetTablePluginInfo& DependentPluginInfo = AssetTable.GetPluginInfoByIndex(DependentPluginIndex);
				TSharedPtr<FPluginAndDependenciesGroupNode> PluginGroup = MakeShared<FPluginAndDependenciesGroupNode>(PluginGroupName, GetAssetTableWeak(), DependentPluginIndex);
				PluginGroup->SetAuthorGrouping(GetAuthorGrouping());
				PluginGroup->CreateChildren()->AddAssetChildrenNodes();
				AddChildAndSetParent(PluginGroup);
			}
		}
	}

	bAreChildrenCreated = true;
	return true;
}

////////////////////////////////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
