// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#include "AssetTable.h"
#include "InsightsCore/Table/ViewModels/TableTreeNode.h"

////////////////////////////////////////////////////////////////////////////////////////////////////

class FPluginSimpleGroupNode;

class FAssetTreeNode : public UE::Insights::FTableTreeNode
{
	INSIGHTS_DECLARE_RTTI(FAssetTreeNode, UE::Insights::FTableTreeNode)

public:
	/** Initialization constructor for the asset node. */
	explicit FAssetTreeNode(const FName InName, TWeakPtr<FAssetTable> InParentTable, int32 InRowIndex)
		: FTableTreeNode(InName, InParentTable, InRowIndex)
		, AssetTablePtr(InParentTable.Pin().Get())
	{
	}

	/** Initialization constructor for the group node. */
	explicit FAssetTreeNode(const FName InGroupName, TWeakPtr<FAssetTable> InParentTable)
		: FTableTreeNode(InGroupName, InParentTable)
		, AssetTablePtr(InParentTable.Pin().Get())
	{
	}

	/** Initialization constructor for the asset and/or group node. */
	explicit FAssetTreeNode(const FName InName, TWeakPtr<FAssetTable> InParentTable, int32 InRowIndex, bool bIsGroup)
		: FTableTreeNode(InName, InParentTable, InRowIndex, bIsGroup)
		, AssetTablePtr(InParentTable.Pin().Get())
	{
	}

	virtual ~FAssetTreeNode() = default;

	TWeakPtr<FAssetTable> GetAssetTableWeak() const { return StaticCastWeakPtr<FAssetTable>(GetParentTable()); }

	bool IsValidAsset() const { return AssetTablePtr && AssetTablePtr->IsValidRowIndex(RowId.RowIndex); }
	FAssetTable& GetAssetTableChecked() const { return *AssetTablePtr; }
	const FAssetTableRow& GetAssetChecked() const { return AssetTablePtr->GetAssetChecked(RowId.RowIndex); }

	virtual const FSlateBrush* GetIcon() const final;
	virtual FLinearColor GetIconColor() const final;
	virtual FLinearColor GetColor() const final;

	bool IsPluginRelatedNode() const { return Is<FPluginSimpleGroupNode>(); }
	bool IsAssetRelatedNode() const { return !IsPluginRelatedNode(); }

protected:

	// Set of UI style options for node types
	enum class EStyle
	{
		EDefault,
		EGroup,
		EAsset,
		EDependencies,
		EPlugin,
	};

	virtual EStyle GetStyle() const;
	const FSlateBrush* GetIcon(EStyle Style) const;
	FLinearColor GetIconColor(EStyle Style) const;
	FLinearColor GetColor(EStyle Style) const;

private:
	FAssetTable* AssetTablePtr;
};

/** Type definition for shared pointers to instances of FAssetTreeNode. */
typedef TSharedPtr<class FAssetTreeNode> FAssetTreeNodePtr;

/** Type definition for shared references to instances of FAssetTreeNode. */
typedef TSharedRef<class FAssetTreeNode> FAssetTreeNodeRef;

/** Type definition for shared references to const instances of FAssetTreeNode. */
typedef TSharedRef<const class FAssetTreeNode> FAssetTreeNodeRefConst;

/** Type definition for weak references to instances of FAssetTreeNode. */
typedef TWeakPtr<class FAssetTreeNode> FAssetTreeNodeWeak;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FAssetDependenciesGroupTreeNode : public FAssetTreeNode
{
	INSIGHTS_DECLARE_RTTI(FAssetDependenciesGroupTreeNode, FAssetTreeNode)

public:
	/** Initialization constructor for the group node. */
	explicit FAssetDependenciesGroupTreeNode(const FName InGroupName, TWeakPtr<FAssetTable> InParentTable, int32 InParentRowIndex)
		: FAssetTreeNode(InGroupName, InParentTable, InParentRowIndex, true)
		, bAreChildrenCreated(false)
		, AuthorGrouping(nullptr)
	{
		// Initially collapsed. Lazy create children when first expanded.
		SetExpansion(false);
	}

	virtual ~FAssetDependenciesGroupTreeNode() = default;

	virtual EStyle GetStyle() const override;

	virtual const FText GetExtraDisplayName() const;
	virtual bool OnLazyCreateChildren(TSharedPtr<class UE::Insights::STableTreeView> InTableTreeView) override;

	virtual const UE::Insights::FTreeNodeGrouping* GetAuthorGrouping() const override { return AuthorGrouping; }
	void SetAuthorGrouping(const UE::Insights::FTreeNodeGrouping* InAuthorGrouping) { AuthorGrouping = InAuthorGrouping; }

private:
	bool bAreChildrenCreated;
	const UE::Insights::FTreeNodeGrouping* AuthorGrouping;
};

/** Type definition for shared pointers to instances of FAssetDependenciesGroupTreeNode. */
typedef TSharedPtr<class FAssetDependenciesGroupTreeNode> FAssetDependenciesGroupTreeNodePtr;

////////////////////////////////////////////////////////////////////////////////////////////////////

class FPluginSimpleGroupNode : public FAssetTreeNode
{
	INSIGHTS_DECLARE_RTTI(FPluginSimpleGroupNode, FAssetTreeNode)

public:
	/** Initialization constructor. */
	explicit FPluginSimpleGroupNode(const FName InGroupName, TWeakPtr<FAssetTable> InParentTable, int32 InPluginIndex)
		: FAssetTreeNode(InGroupName, InParentTable)
		, PluginIndex(InPluginIndex)
		, AuthorGrouping(nullptr)
	{
	}

	virtual ~FPluginSimpleGroupNode() = default;

	virtual EStyle GetStyle() const override;

	void AddAssetChildrenNodes();

	int32 GetPluginIndex() const { return PluginIndex; }

	virtual const UE::Insights::FTreeNodeGrouping* GetAuthorGrouping() const override { return AuthorGrouping; }
	void SetAuthorGrouping(const UE::Insights::FTreeNodeGrouping* InAuthorGrouping) { AuthorGrouping = InAuthorGrouping; }

protected:
	int32 PluginIndex;
	const UE::Insights::FTreeNodeGrouping* AuthorGrouping;
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FPluginAndDependenciesGroupNode : public FPluginSimpleGroupNode
{
	INSIGHTS_DECLARE_RTTI(FPluginAndDependenciesGroupNode, FPluginSimpleGroupNode)

public:
	/** Initialization constructor. */
	explicit FPluginAndDependenciesGroupNode(const FName InGroupName, TWeakPtr<FAssetTable> InParentTable, int32 InPluginIndex)
		: FPluginSimpleGroupNode(InGroupName, InParentTable, InPluginIndex)
	{
	}

	virtual ~FPluginAndDependenciesGroupNode() = default;

	virtual EStyle GetStyle() const override;

	TSharedPtr<FPluginSimpleGroupNode> CreateChildren();
};

////////////////////////////////////////////////////////////////////////////////////////////////////

class FPluginDependenciesGroupNode : public FPluginSimpleGroupNode
{
	INSIGHTS_DECLARE_RTTI(FPluginDependenciesGroupNode, FPluginSimpleGroupNode)

public:
	/** Initialization constructor. */
	explicit FPluginDependenciesGroupNode(const FName InGroupName, TWeakPtr<FAssetTable> InParentTable, int32 InPluginIndex)
		: FPluginSimpleGroupNode(InGroupName, InParentTable, InPluginIndex)
		, bAreChildrenCreated(false)
	{
	}

	virtual ~FPluginDependenciesGroupNode() = default;

	virtual EStyle GetStyle() const override;

	virtual const FText GetExtraDisplayName() const;
	virtual bool OnLazyCreateChildren(TSharedPtr<class UE::Insights::STableTreeView> InTableTreeView) override;

private:
	bool bAreChildrenCreated;
};

////////////////////////////////////////////////////////////////////////////////////////////////////
