// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CookMetadata.h"
#include "Delegates/DelegateCombinations.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "AssetManagerEditorModule.h"

#include "TreeView/AssetTable.h"
#include "InsightsCore/Table/Widgets/STableTreeView.h"

class FAssetTreeNode;

class SAssetTableTreeView : public UE::Insights::STableTreeView
{
public:
	DECLARE_DELEGATE_OneParam(FOnSelectionChanged, const TArray<TSharedPtr<UE::Insights::FTableTreeNode>>)

public:
	/** Default constructor. */
	SAssetTableTreeView();

	/** Virtual destructor. */
	virtual ~SAssetTableTreeView();

	SLATE_BEGIN_ARGS(SAssetTableTreeView)
		: _OnSelectionChanged()
	{
	}
	SLATE_EVENT(FOnSelectionChanged, OnSelectionChanged)

	SLATE_END_ARGS()

	/**
	 * Construct this widget
	 * @param InArgs - The declaration data for this widget
	 */
	void Construct(const FArguments& InArgs, TSharedPtr<FAssetTable> InTablePtr);

	virtual void ConstructHeaderArea(TSharedRef<SVerticalBox> InWidgetContent);
	virtual TSharedPtr<SWidget> ConstructFooter() override;

	TSharedPtr<FAssetTable> GetAssetTable() { return StaticCastSharedPtr<FAssetTable>(GetTable()); }
	const TSharedPtr<FAssetTable> GetAssetTable() const { return StaticCastSharedPtr<FAssetTable>(GetTable()); }

	virtual void Reset();

	/**
	 * Ticks this widget.  Override in derived classes, but always call the parent implementation.
	 *
	 * @param  AllottedGeometry The space allotted for this widget
	 * @param  InCurrentTime  Current absolute real time
	 * @param  InDeltaTime  Real time passed since last tick
	 */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	/**
	 * Rebuilds the tree (if necessary).
	 * @param bResync - If true, it forces a resync even if the list did not changed since last sync.
	 */
	virtual void RebuildTree(bool bResync);

	/**
	 * Sets the search filter to the given name (does not check if exists but UI is clear when no data matches filter)
	 * @param Name - Could be an asset name or a plugin name (any name in the tree really)
	 */
	void SetFilterToName(const FString& Name);

	////////////////////////////////////////////////////////////////////////////////////////////////////
	// IAsyncOperationStatusProvider implementation

	virtual bool IsRunning() const override;
	virtual double GetAllOperationsDuration() override;
	virtual FText GetCurrentOperationName() const override;

	////////////////////////////////////////////////////////////////////////////////////////////////////

private:
	void ExecuteFindInContentBrowserAction();
	bool CanExecuteFindInContentBrowserAction();

	enum class ECheckFilesExistAndHashMatchesResult : uint8
	{
		Okay,
		RegistryDoesNotExist,
		CookMetadataDoesNotExist,
		FailedToHashRegistry,
		FailedToLoadCookMetadata,
		FailedToLoadRegistry,
		HashesDoNotMatch,
		Unknown
	};

	virtual void InternalCreateGroupings() override;

	virtual void ExtendMenu(FMenuBuilder& MenuBuilder) override;

	void RequestOpenRegistry();
	void OpenRegistry();

	ECheckFilesExistAndHashMatchesResult CheckFilesExistAndHashMatches(const FString& MetadataFilename, const FString& RegistryFilename, UE::Cook::FCookMetadataState& MetadataTemporaryStorage);

	FText GetOpenRegistryToolTipText() const;
	bool CanChangeRegistry() const;
	FReply OnClickedOpenRegistry();

	FText GetFooterLeftText() const;
	FText GetFooterCenterText1() const;
	FText GetFooterCenterText2() const;
	FText GetFooterRightText1() const;

	virtual void TreeView_OnSelectionChanged(UE::Insights::FTableTreeNodePtr SelectedItem, ESelectInfo::Type SelectInfo) override;

	virtual TSharedRef<SWidget> ConstructSearchBox() override;

	void InitAvailableViewPresets();

	void ExportDependencyData() const;

	TArray<FAssetData> GetAssetDataForSelection() const;

	void CalculateBaseAndMarginalCostForSelection(TSet<int32>& SelectionSetIndices, int64* OutTotalSizeMultiplyUsed, int64* OutTotalSizeSingleUse) const;

	/** Populates an asset table row (view model for the asset tree view) with asset data (only identification data; type+path+name). */
	void QuickPopulateAssetTableRow(class FAssetTableRow& OutRow, const FAssetData& AssetData, class FAssetTable& AssetTable) const;

	/** Populates an asset table row (view model for the asset tree view) with asset data. */
	void PopulateAssetTableRow(class FAssetTableRow& OutRow, const FAssetData& AssetData, class FAssetTable& AssetTable) const;

	bool IsRegistrySourceValid() const { return RegistrySource.GetOwnedRegistryState() != nullptr; }

	void RequestRefreshAssets();
	void RefreshAssets();

	void ClearTableAndTree();

	typedef TSet<const TCHAR*, TStringPointerSetKeyFuncs_DEPRECATED<const TCHAR*>> DeprecatedTCharSetType;
	void DumpDifferencesBetweenDiscoveredDataAndLoadedMetadata(TMap<const TCHAR*, DeprecatedTCharSetType, FDefaultSetAllocator, TStringPointerMapKeyFuncs_DEPRECATED<const TCHAR*, DeprecatedTCharSetType>>& DiscoveredPluginDependencyEdges) const;

	void RequestRebuildTree(bool bNeedsColumnRebuild = false);

	void UpdateRegistryInfoTextPostLoad(ECheckFilesExistAndHashMatchesResult StatusResult);

private:
	bool bNeedsToOpenRegistry = false;
	bool bNeedsToRefreshAssets = false;
	bool bNeedsToRebuild = false;
	bool bNeedsToRebuildColumns = false;

	FText FooterLeftTextStoredPreOpen;
	FText RegistryLoadedText;
	FText FooterLeftText;
	FText FooterCenterText1;
	FText FooterCenterText2;
	FText FooterRightText1;

	/** Delegate to invoke when selection changes. */
	FOnSelectionChanged OnSelectionChanged;
	TSet<int32> SelectedAssetIndices;
	TSet<int32> SelectedPluginIndices;

	/** List of valid registry sources */
	TArray<TSharedPtr<FString>> SourceComboList;

	/** Used so we can set the search filter from code and have it reflected in the UI */
	TSharedPtr<SWidget> SearchBoxWidget;

	/**
	* Text block where we put the timestamp for the asset registry so the user knows
	* if they are looking at super out of date data.
	*/
	TSharedPtr<STextBlock> RegistryInfoText;

	FAssetManagerEditorRegistrySource RegistrySource;

	UE::Cook::FCookMetadataState CookMetadata;

	/** Cached interfaces */
	class UAssetManager* AssetManager;
	class IAssetManagerEditorModule* EditorModule;
};
