// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AssetRegistry/PathTree.h"
#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"
#include "UObject/PrimaryAssetId.h"
#include "Interfaces/ITargetPlatform.h"
#include "AssetRegistry/AssetRegistryState.h"
#include "Engine/AssetManagerTypes.h"
#include "CollectionManagerTypes.h"

#define UE_API ASSETMANAGEREDITOR_API

class ICollectionContainer;
class FUICommandList;
class FUICommandInfo;
/**
 * AssetRegistry Dependency Category + QueryFlags, to pass around the two enums in a single variable.
 */
struct FAssetManagerDependencyQuery
{
	UE::AssetRegistry::EDependencyCategory Categories = UE::AssetRegistry::EDependencyCategory::All;
	UE::AssetRegistry::EDependencyQuery Flags = UE::AssetRegistry::EDependencyQuery::NoRequirements;

	bool IsNone() const
	{
		return Categories == UE::AssetRegistry::EDependencyCategory::None;
	}

	FAssetManagerDependencyQuery() = default;

	static FAssetManagerDependencyQuery All()
	{
		return FAssetManagerDependencyQuery();
	}
	static FAssetManagerDependencyQuery None()
	{
		FAssetManagerDependencyQuery Result;
		Result.Categories = UE::AssetRegistry::EDependencyCategory::None;
		return Result;
	}
};

DECLARE_LOG_CATEGORY_EXTERN(LogAssetManagerEditor, Log, All);

DECLARE_DELEGATE_RetVal(FText, FOnGetPrimaryAssetDisplayText);
DECLARE_DELEGATE_OneParam(FOnSetPrimaryAssetType, FPrimaryAssetType);
DECLARE_DELEGATE_OneParam(FOnSetPrimaryAssetId, FPrimaryAssetId);

class SToolTip;
class SWidget;
struct FAssetData;

/** Struct containing the information about currently investigated asset context */
struct FAssetManagerEditorRegistrySource
{
	/** Display name of the source, equal to target Platform, Editor, or Custom */
	FString SourceName;

	/** Filename registry was loaded from */
	FString SourceFilename;

	/** 
	* Timestamp for the registry we loaded from, if from a file. Modified time if the OS supports it,
	* otherwise creation time. If we can't get anything or not from a file, empty string. In local timezone.
	*/
	FString SourceTimestamp;

	/** Target platform for this state, may be null */
	ITargetPlatform* TargetPlatform;

private:
	/** Raw asset registry state, only valid if bIsEditor is false */
	const FAssetRegistryState* RegistryStatePrivate;
	FPathTree CachedRegistryStatePathTree;
	/** Pointer to the real AssetRegistry, only valid if bIsEditor is true */
	const IAssetRegistry* AssetRegistryPrivate;

public:
	/** If true, this is the editor  */
	uint8 bIsEditor : 1;

	/** If true, management data has been initialized */
	uint8 bManagementDataInitialized : 1;

	/** Map of chunk information, from chunk id to primary assets/specific assets that are explicitly assigned to that chunk */
	TMap<int32, FAssetManagerChunkInfo> ChunkAssignments;

	/** If SourceName matches this, this is the editor */
	static UE_API const FString EditorSourceName;

	/** If SourceName matches this, this is the custom source loaded from an arbitrary path */
	static UE_API const FString CustomSourceName;

	FAssetManagerEditorRegistrySource()
		: TargetPlatform(nullptr)
		, RegistryStatePrivate(nullptr)
		, AssetRegistryPrivate(nullptr)
		, bIsEditor(false)
		, bManagementDataInitialized(false)
	{
	}

	~FAssetManagerEditorRegistrySource()
	{
		ClearRegistry();
	}

	// We delete the copy constructor because it's not clear whether we should copy RegistryStatePrivate
	FAssetManagerEditorRegistrySource(const FAssetManagerEditorRegistrySource& Other) = delete;
	FAssetManagerEditorRegistrySource& operator=(const FAssetManagerEditorRegistrySource& Other) = delete;

	FAssetManagerEditorRegistrySource& operator=(FAssetManagerEditorRegistrySource&& Other)
	{
		SourceName = MoveTemp(Other.SourceName);
		SourceFilename = MoveTemp(Other.SourceFilename);
		TargetPlatform = MoveTemp(Other.TargetPlatform);
		bManagementDataInitialized = Other.bManagementDataInitialized;
		Other.bManagementDataInitialized = false;
		ChunkAssignments = MoveTemp(Other.ChunkAssignments);

		if (Other.bIsEditor)
		{
			SetUseEditorAssetRegistry(Other.AssetRegistryPrivate);
		}
		else
		{
			SetRegistryState(Other.RegistryStatePrivate);
			CachedRegistryStatePathTree = MoveTemp(Other.CachedRegistryStatePathTree);
		}
		// Remove the Other's RegistryState without deleting it
		Other.RegistryStatePrivate = nullptr;
		Other.CachedRegistryStatePathTree = FPathTree();
		Other.AssetRegistryPrivate = nullptr;
		Other.bIsEditor = false;

		return *this;
	}

	FAssetManagerEditorRegistrySource(FAssetManagerEditorRegistrySource&& Other)
		:FAssetManagerEditorRegistrySource()
	{
		*this = MoveTemp(Other);
	}

	/** Returns the RegistryState owned by this. Will be null if this does not have a Registry or is using the global editor Registry */
	const FAssetRegistryState* GetOwnedRegistryState() const
	{
		if (!bIsEditor)
		{
			return RegistryStatePrivate; // May be null
		}
		return nullptr;
	}

	/**
	 * Set the RegistryState viewed by this FAssetManagerEditorRegistrySource to a dynamically allocated FAssetRegistryState
	 * FAssetManagerEditorRegistrySource takes ownership of the given AssetRegistry and deletes it is when cleared.
	 * Do not call with  AssetRegistry->GetAssetRegistryState; use SetUseEditorAssetRegistry(AssetRegistry) instead.
	 */
	void SetRegistryState(const FAssetRegistryState* InRegistryState)
	{
		ClearRegistry();

		if (InRegistryState)
		{
			bIsEditor = false;
			RegistryStatePrivate = InRegistryState;
			// Gather all sub-paths by enumerating known paths in the registry
			RegistryStatePrivate->EnumerateAllPaths([this](FName Path)
				{
					CachedRegistryStatePathTree.CachePath(Path, [](FName AddedPath) {});
				});
		}
	}

	/**
	 * Set the RegistryState viewed by this FAssetManagerEditorRegistrySource to the FAssetRegistryState owned by the given
	 * global AssetRegistry.
	 */
	void SetUseEditorAssetRegistry(const IAssetRegistry* AssetRegistry)
	{
		ClearRegistry();

		if (AssetRegistry)
		{
			bIsEditor = true;
			AssetRegistryPrivate = AssetRegistry;
		}
	}

	bool HasRegistry() const
	{
		return bIsEditor || RegistryStatePrivate != nullptr;
	}

	void ClearRegistry()
	{
		if (!bIsEditor)
		{
			delete RegistryStatePrivate; // May be null
		}
		bIsEditor = false;
		RegistryStatePrivate = nullptr;
		CachedRegistryStatePathTree = FPathTree();
		AssetRegistryPrivate = nullptr;
	}

	// grabs the SourceTimestamp from the asset registry file changed times.
	UE_API void LoadRegistryTimestamp();

	// Functions to forward on to the RegistryState, either the owned one or the editor's global State.
	FAssetData GetAssetByPackageName(FName PackageName) const
	{
		if (bIsEditor)
		{
			TArray<FAssetData> Assets;
			AssetRegistryPrivate->GetAssetsByPackageName(PackageName, Assets, true /* bIncludeOnlyOnDiskAssets */);
			return Assets.Num() > 0 ? Assets[0] : FAssetData();
		}
		else if (RegistryStatePrivate)
		{
			TArray<const FAssetData*> Assets = RegistryStatePrivate->CopyAssetsByPackageName(PackageName);
			return Assets.Num() > 0 ? *Assets[0] : FAssetData();
		}
		return FAssetData();
	}

	FAssetData GetAssetByObjectPath(const FSoftObjectPath& ObjectPath) const
	{
		if (bIsEditor)
		{
			return AssetRegistryPrivate->GetAssetByObjectPath(ObjectPath, true /* bIncludeOnlyOnDiskAssets */);
		}
		else if (RegistryStatePrivate)
		{
			const FAssetData* AssetData = RegistryStatePrivate->GetAssetByObjectPath(ObjectPath);
			return AssetData ? *AssetData : FAssetData();
		}
		else
		{
			return nullptr;
		}
	}

	template <typename ValueType>
	bool GetAssetTagByObjectPath(const FSoftObjectPath& InObjectPath, FName InTagName, ValueType& OutTagValue) const
	{
		if (bIsEditor)
		{
			return AssetRegistryPrivate->GetAssetByObjectPath(InObjectPath, true /* bIncludeOnlyOnDiskAssets */).GetTagValue(InTagName, OutTagValue);
		}
		else if (RegistryStatePrivate)
		{
			const FAssetData* AssetData = RegistryStatePrivate->GetAssetByObjectPath(InObjectPath);
			if (AssetData)
			{
				return AssetData->GetTagValue(InTagName, OutTagValue);
			}
			return false;
		}
		return false;
	}

	bool GetDependencies(const FAssetIdentifier& AssetIdentifier, TArray<FAssetIdentifier>& OutDependencies,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const
	{
		if (bIsEditor)
		{
			return AssetRegistryPrivate->GetDependencies(AssetIdentifier, OutDependencies, Category, Flags);
		}
		else if (RegistryStatePrivate)
		{
			return RegistryStatePrivate->GetDependencies(AssetIdentifier, OutDependencies, Category, Flags);
		}
		else
		{
			return false;
		}
	}

	bool GetReferencers(const FAssetIdentifier& AssetIdentifier, TArray<FAssetIdentifier>& OutReferencers,
		UE::AssetRegistry::EDependencyCategory Category = UE::AssetRegistry::EDependencyCategory::All,
		const UE::AssetRegistry::FDependencyQuery& Flags = UE::AssetRegistry::FDependencyQuery()) const
	{
		if (bIsEditor)
		{
			return AssetRegistryPrivate->GetReferencers(AssetIdentifier, OutReferencers, Category, Flags);
		}
		else if (RegistryStatePrivate)
		{
			return RegistryStatePrivate->GetReferencers(AssetIdentifier, OutReferencers, Category, Flags);
		}
		else
		{
			return false;
		}
	}

	TOptional<FAssetPackageData> GetAssetPackageDataCopy(FName PackageName) const
	{
		if (bIsEditor)
		{
			return AssetRegistryPrivate->GetAssetPackageDataCopy(PackageName);
		}
		else if (RegistryStatePrivate)
		{
			const FAssetPackageData* DataPtr = RegistryStatePrivate->GetAssetPackageData(PackageName);
			return DataPtr ? TOptional<FAssetPackageData>(*DataPtr) : TOptional<FAssetPackageData>();
		}
		else
		{
			return TOptional<FAssetPackageData>();
		}
	}

	bool GetAssets(const FARFilter& Filter, TArray<FAssetData>& OutAssets) const
	{
		if (bIsEditor)
		{
			return AssetRegistryPrivate->GetAssets(Filter, OutAssets);
		}
		else if (RegistryStatePrivate)
		{
			FARCompiledFilter CompiledFilter;
			CompiledFilter.PackageNames.Append(Filter.PackageNames);
			CompiledFilter.bIncludeOnlyOnDiskAssets = Filter.bIncludeOnlyOnDiskAssets;
			
			if (Filter.PackagePaths.Num() > 0)
			{
				if (Filter.bRecursivePaths)
				{
					// Add the sub-paths of all the input paths to the expanded list
					for (const FName& PackagePath : Filter.PackagePaths)
					{
						CachedRegistryStatePathTree.GetSubPaths(PackagePath, CompiledFilter.PackagePaths);
					}
				}
				else
				{
					for (const FName& Path : Filter.PackagePaths)
					{
						CompiledFilter.PackagePaths.Add(Path);
					}
				}
			}

			RegistryStatePrivate->GetAssets(CompiledFilter, {}, OutAssets);
			return true;
		}
		return false;
	}

};

/**
 * The Asset Manager Editor module handles creating UI for asset management and exposes several commands
 */
class IAssetManagerEditorModule : public IModuleInterface
{
public:
	struct FPrimaryAssetIdSelectorArgs
	{
		FOnGetPrimaryAssetDisplayText OnGetDisplayText;
		FOnGetPrimaryAssetDisplayText OnGetTooltipText;
		FOnSetPrimaryAssetId OnSetId;
		bool bAllowClear = true;
		TArray<FPrimaryAssetType> AllowedTypes;
		TArray<const UClass*> AllowedClasses;
		TArray<const UClass*> DisallowedClasses;
	};

public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IAssetManagerEditorModule& Get()
	{
		return FModuleManager::LoadModuleChecked< IAssetManagerEditorModule >( "AssetManagerEditor" );
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded( "AssetManagerEditor" );
	}

	/** 
	 * Creates a simple version of a Primary Asset Type selector, not bound to a PropertyHandle 
	 * @param OnGetDisplayText Delegate that returns the text to display in body of combo box
	 * @param OnSetType Delegate called when type is changed
	 * @param bAllowClear If true, add None option to top
	 * @param bAlowAll If true, add All Types option to bottom, returns AllPrimaryAssetTypes if selected
	 */
	static UE_API TSharedRef<SWidget> MakePrimaryAssetTypeSelector(FOnGetPrimaryAssetDisplayText OnGetDisplayText, FOnSetPrimaryAssetType OnSetType, bool bAllowClear = true, bool bAllowAll = false);

	/** 
	 * Creates a simple version of a Primary Asset Id selector, not bound to a PropertyHandle
	 * @param OnGetDisplayText Delegate that returns the text to display in body of combo box
	 * @param OnSetId Delegate called when id is changed
	 * @param bAllowClear If true, add None option to top
	 */
	static UE_API TSharedRef<SWidget> MakePrimaryAssetIdSelector(FOnGetPrimaryAssetDisplayText OnGetDisplayText, FOnSetPrimaryAssetId OnSetId, bool bAllowClear = true, 
		TArray<FPrimaryAssetType> AllowedTypes = TArray<FPrimaryAssetType>(), TArray<const UClass*> AllowedClasses = TArray<const UClass*>(), TArray<const UClass*> DisallowedClasses = TArray<const UClass*>()
	);

	/**
	 * Creates a simple version of a Primary Asset Id selector, not bound to a PropertyHandle
	 * @param PrimaryAssetIdSelectorArgs struct of arguments to build the PrimaryAssetIdSelector
	 */
	static UE_API TSharedRef<SWidget> MakePrimaryAssetIdSelector(FPrimaryAssetIdSelectorArgs PrimaryAssetIdSelectorArgs);

	/** Called to get list of valid primary asset types */
	static UE_API void GeneratePrimaryAssetTypeComboBoxStrings(TArray< TSharedPtr<FString> >& OutComboBoxStrings, TArray<TSharedPtr<SToolTip>>& OutToolTips, TArray<bool>& OutRestrictedItems, bool bAllowClear, bool bAllowAll);

	/** Called for asset picker to see rather to show asset */
	static UE_API bool OnShouldFilterPrimaryAsset(const FAssetData& InAssetData, TArray<FPrimaryAssetType> AllowedTypes);

	/** Creates fake AssetData to represent chunks */
	static UE_API FAssetData CreateFakeAssetDataFromChunkId(int32 ChunkID);

	/** Returns >= 0 if this is a fake chunk AssetData, otherwise INDEX_NONE */
	static UE_API int32 ExtractChunkIdFromFakeAssetData(const FAssetData& InAssetData);

	/** Creates fake AssetData to represent a primary asset */
	static UE_API FAssetData CreateFakeAssetDataFromPrimaryAssetId(const FPrimaryAssetId& PrimaryAssetId);

	/** Returns valid PrimaryAssetId if this is a fake AssetData for a primary asset */
	static UE_API FPrimaryAssetId ExtractPrimaryAssetIdFromFakeAssetData(const FAssetData& InAssetData);

	/** Parses a list of AssetData and extracts AssetIdentifiers, handles the fake asset data */
	static UE_API void ExtractAssetIdentifiersFromAssetDataList(const TArray<FAssetData>& AssetDataList, TArray<FAssetIdentifier>& OutAssetIdentifiers);

	/** Fake package path used by placeholder asset data, has primary asset ID added to it to set package name */
	static UE_API const FName PrimaryAssetFakeAssetDataPackagePath;

	/** Specific package path inside PrimaryAssetFakeAssetDataPackagePath representing chunks */
	static UE_API const FName ChunkFakeAssetDataPackageName;
	
	/** Fake type returned from type combo box to mean all types */
	static UE_API const FPrimaryAssetType AllPrimaryAssetTypes;

	/** Custom column names */
	static UE_API const FName ResourceSizeName;
	static UE_API const FName DiskSizeName;
	static UE_API const FName ManagedResourceSizeName;
	static UE_API const FName ManagedDiskSizeName;
	static UE_API const FName TotalUsageName;
	static UE_API const FName CookRuleName;
	static UE_API const FName ChunksName;
	static UE_API const FName PluginName;

	/** Gets the value of a "virtual" column for an asset data, this will query the AssetManager for you and takes current platform into account. Returns true and sets out parameter if found */
	virtual bool GetStringValueForCustomColumn(const FAssetData& AssetData, FName ColumnName, FString& OutValue, const FAssetManagerEditorRegistrySource* OverrideRegistrySource = nullptr) = 0;

	/** Gets a display text value for a virtual column. Returns true and sets out parameter if found */
	virtual bool GetDisplayTextForCustomColumn(const FAssetData& AssetData, FName ColumnName, FText& OutValue, const FAssetManagerEditorRegistrySource* OverrideRegistrySource = nullptr) = 0;

	/** Gets the value of a "virtual" column for an asset data, this will query the AssetManager for you and takes current platform into account. Returns true and sets out parameter if found */
	virtual bool GetIntegerValueForCustomColumn(const FAssetData& AssetData, FName ColumnName, int64& OutValue, const FAssetManagerEditorRegistrySource* OverrideRegistrySource = nullptr) = 0;

	/** Returns the set of asset packages managed the given asset data, returns true if any found. This handles the fake chunk/primary asset types above */
	virtual bool GetManagedPackageListForAssetData(const FAssetData& AssetData, TSet<FName>& ManagedPackageSet) = 0;

	/** Gets the list of registry sources that are available */
	virtual void GetAvailableRegistrySources(TArray<const FAssetManagerEditorRegistrySource*>& AvailableSources) = 0;

	/** Returns the currently selected registry source. If bNeedManagementData is set it it will cache the chunk/management information */
	virtual const FAssetManagerEditorRegistrySource* GetCurrentRegistrySource(bool bNeedManagementData = false) = 0;

	/** Sets the current registry source, this loads the asset registry state if needed and may spawn a file load dialog for custom */
	virtual void SetCurrentRegistrySource(const FString& SourceName) = 0;

	/** Sets up a registry source. If InOutRegistrySource->SourceName is CustomSourceName, a dialog will be presented to select the asset registry to load. If OptInFilePath is provided, it will be used instead of opening a dialog. */
	virtual bool PopulateRegistrySource(FAssetManagerEditorRegistrySource* InOutRegistrySource, const FString* OptInFilePath = nullptr) = 0;

	/** Refreshes the management dictionary and all sources */
	virtual void RefreshRegistryData() = 0;

	/** Returns true if this package exists in the current registry source, optionally setting a redirected package name */
	virtual bool IsPackageInCurrentRegistrySource(FName PackageName) = 0;

	/** Filters list of identifiers and removes ones that do not exist in this registry source. Handles replacing redirectors as well */
	virtual bool FilterAssetIdentifiersForCurrentRegistrySource(TArray<FAssetIdentifier>& AssetIdentifiers,	const FAssetManagerDependencyQuery& DependencyQuery = FAssetManagerDependencyQuery::None(), bool bForwardDependency = true) = 0;

	/** Creates a collection in the project collection container from a list of packages, will overwrite/modify an existing collection of the same name. Will display feedback to the user if bShowFeedback is true */
	UE_DEPRECATED(5.6, "Use the ICollectionContainer overload instead.")
	virtual bool WriteCollection(FName CollectionName, ECollectionShareType::Type ShareType, const TArray<FName>& PackageNames, bool bShowFeedback = true) = 0;

	/** Creates a collection from a list of packages, will overwrite/modify an existing collection of the same name. Will display feedback to the user if bShowFeedback is true */
	virtual bool WriteCollection(ICollectionContainer& CollectionContainer, FName CollectionName, ECollectionShareType::Type ShareType, const TArray<FName>& PackageNames, bool bShowFeedback = true) = 0;

	/** Opens asset management UI, with selected assets. Pass as value so it can be used in delegates */
	virtual void OpenAssetAuditUI(TArray<FAssetData> SelectedAssets) = 0;
	virtual void OpenAssetAuditUI(TArray<FAssetIdentifier> SelectedIdentifiers) = 0;
	virtual void OpenAssetAuditUI(TArray<FName> SelectedPackages) = 0;

	/** Spawns reference viewer, showing selected packages or identifiers */
	virtual void OpenReferenceViewerUI(const TArray<FAssetIdentifier> SelectedIdentifiers, const FReferenceViewerParams ReferenceViewerParams = FReferenceViewerParams()) = 0;
	virtual void OpenReferenceViewerUI(const TArray<FName> SelectedPackages, const FReferenceViewerParams ReferenceViewerParams = FReferenceViewerParams()) = 0;

	/** Returns true if Reference Viewer UI is available and has selected nodes. If true, will also append the asset data for the selected nodes into the passed array*/
	virtual bool GetReferenceViewerUISelectedNodesAssetData(TArray<FAssetData>& OutAssetDatas) = 0;

	/** Returns false if should not open reference viewer for items */
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FCanOpenReferenceViewerUI, const TArray<FAssetIdentifier>&, FText*);
	virtual FCanOpenReferenceViewerUI& OnCanOpenReferenceViewerUI() = 0;

	/** Spawns size map with selected packages */
	virtual void OpenSizeMapUI(TArray<FAssetIdentifier> SelectedIdentifiers) = 0;
	virtual void OpenSizeMapUI(TArray<FName> SelectedPackages) = 0;

	/** Open the Shader cook stats */
	virtual void OpenShaderCookStatistics(TArray<FName> SelectedIdentifiers) = 0;

	/** Open the Asset Disk Size Tree View*/
	virtual void OpenAssetSizeTreeViewUI(const FString& SearchTerm) = 0;

	/** Called by SReferenceViewer to update given list of commands with any of the registered delegates */
	virtual void AppendRegisteredCommandsToCommandList(TSharedRef<FUICommandList> ListToAppendTo) = 0;

	/** Passes commandlist and command info array to be updated by any of the registered delegates. Broadcast from AppendRegisteredCommandsToCommandList */
	DECLARE_MULTICAST_DELEGATE_TwoParams(FRegisterAdditionalReferenceViewerCommands, TSharedRef<FUICommandList>, TArray<TSharedPtr<FUICommandInfo>>&);
	virtual FRegisterAdditionalReferenceViewerCommands &OnRegisterAdditionalReferenceViewerCommands() = 0;

	/** Returns our list of command info populated by OnRegisterAdditionalReferenceViewerCommands. Needed to be tracked separately from command list to populate commands in menu ui */
	virtual const TArray<TSharedPtr<FUICommandInfo>>& GetRegisteredAdditionalReferenceViewerCommands() = 0;
	
	/* Return a UToolsMenu generated widget containing the asset source control context menu. */
	virtual TSharedRef<SWidget> CreateAssetSourceControlContextMenu(const TArray<FAssetData>& SelectedAssets) = 0;
};

#undef UE_API
