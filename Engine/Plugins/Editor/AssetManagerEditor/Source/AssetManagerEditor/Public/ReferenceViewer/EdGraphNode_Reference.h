// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "EdGraph/EdGraphNode.h"
#include "AssetRegistry/AssetData.h"
#include "EdGraph_ReferenceViewer.h"
#include "EdGraphNode_Reference.generated.h"

#define UE_API ASSETMANAGEREDITOR_API

class UEdGraphPin;

UCLASS(MinimalAPI)
class UEdGraphNode_Reference : public UEdGraphNode
{
	GENERATED_UCLASS_BODY()

	/** Returns first asset identifier */
	UE_API FAssetIdentifier GetIdentifier() const;
	
	/** Returns all identifiers on this node including virtual things */
	UE_API void GetAllIdentifiers(TArray<FAssetIdentifier>& OutIdentifiers) const;

	/** Returns only the packages in this node, skips searchable names */
	UE_API void GetAllPackageNames(TArray<FName>& OutPackageNames) const;

	/** Returns our owning graph */
	UE_API UEdGraph_ReferenceViewer* GetReferenceViewerGraph() const;

	// UEdGraphNode implementation
	UE_API virtual FText GetNodeTitle(ENodeTitleType::Type TitleType) const override;
	UE_API FLinearColor GetNodeTitleColor() const override;
	UE_API virtual FText GetTooltipText() const override;
	UE_API virtual void AllocateDefaultPins() override;
	UE_API virtual UObject* GetJumpTargetForDoubleClick() const override;
	UE_API virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;
	virtual bool ShowPaletteIconOnNode() const override { return true; }
	// End UEdGraphNode implementation

	void SetAllowThumbnail(bool bInAllow) { bAllowThumbnail = bInAllow; }
	UE_API bool AllowsThumbnail() const;
	UE_API bool UsesThumbnail() const;
	UE_API bool IsPackage() const;
	UE_API bool IsCollapsed() const;
	bool IsADuplicate() const { return bIsADuplicate; }

	// Nodes that are filtered out still may still show because they
	// are between nodes that pass the filter and the root.  This "filtered"
	// bool allows us to render these in-between nodes differently
	UE_API void SetIsFiltered(bool bInFiltered);
	UE_API bool GetIsFiltered() const;

	bool IsOverflow() const { return bIsOverflow; }

	UE_API const FAssetData& GetAssetData() const;

	UE_API UEdGraphPin* GetDependencyPin();
	UE_API UEdGraphPin* GetReferencerPin();

	UE_INTERNAL UE_API void UpdatePath();

private:
	UE_API void CacheAssetData(const FAssetData& AssetData);
	UE_API void SetupReferenceNode(const FIntPoint& NodeLoc, const TArray<FAssetIdentifier>& NewIdentifiers, const FAssetData& InAssetData, bool bInAllowThumbnail, bool bInIsADuplicate);
	UE_API void SetReferenceNodeCollapsed(const FIntPoint& NodeLoc, int32 InNumReferencesExceedingMax, const TArray<FAssetIdentifier>& Identifiers);
	UE_API void AddReferencer(class UEdGraphNode_Reference* ReferencerNode);

	TArray<FAssetIdentifier> Identifiers;
	FText NodeTitle;

	bool bAllowThumbnail;
	bool bUsesThumbnail;
	bool bIsPackage;
	bool bIsPrimaryAsset;
	bool bIsCollapsed;
	bool bIsADuplicate;
	bool bIsFiltered;
	bool bIsOverflow;

	FAssetData CachedAssetData;
	FLinearColor AssetTypeColor;
	FSlateIcon AssetBrush;

	UEdGraphPin* DependencyPin;
	UEdGraphPin* ReferencerPin;

	friend UEdGraph_ReferenceViewer;
};


#undef UE_API
