// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EdGraph/EdGraphNode.h"
#include "EdGraph_ReferenceViewer.h"
#include "UObject/ObjectMacros.h"
#include "EdGraphNode_ReferencedProperties.generated.h"

#define UE_API ASSETMANAGEREDITOR_API

/**
 * Describes a graph node property referencing an asset from another node
 */
struct FReferencingPropertyDescription
{
	enum class EAssetReferenceType : uint8
	{
		/** Reference comes from a BP Component Type */
		Component,
		/** Reference comes from a BP Variable Type */
		Property,
		/** Reference comes from a BP Variable property Value */
		Value,
		None
	};

	static TSharedRef<FReferencingPropertyDescription> MakeSharedPropertyDescription(
		const FReferencingPropertyDescription& InPropertyDescription
	);

	FReferencingPropertyDescription(const FString& InName, const FString& InReferencerName, const FString& InReferencedNodeName, const EAssetReferenceType& InType, const UClass* InClass, bool bInIsIndirect = false) :
		Name(InName),
		ReferencerName(InReferencerName),
		ReferencedNodeName(InReferencedNodeName),
		Type(InType),
		PropertyClass(InClass),
		bIsIndirectReference(bInIsIndirect)
	{
	}

	bool operator==(const FReferencingPropertyDescription& InOther) const
	{
		return Name == InOther.Name
			&& ReferencerName == InOther.ReferencerName
			&& Type == InOther.Type;
	}

	/** Returns the name of the property */
	const FString& GetName() const { return Name; }

	/** Returns the name of the property referencer */
	const FString& GetReferencerName() const { return ReferencerName; }

	/** Returns the name of the referenced node */
	const FString& GetReferencedNodeName() const { return ReferencedNodeName; }

	/** Returns the property type */
	EAssetReferenceType GetType() const { return Type; }

	/** Returns the property type as a string (useful e.g. for tooltips) */
	FString GetTypeAsString() const;

	const UClass* GetPropertyClass() const
	{
		if (PropertyClass.IsValid())
		{
			return PropertyClass.Get();
		}

		return nullptr;
	}

	bool IsIndirect() const { return bIsIndirectReference; }

private:
	friend class SReferencedPropertyNode;
	FReferencingPropertyDescription() = default;

	FString Name;
	FString ReferencerName;
	FString ReferencedNodeName;
	EAssetReferenceType Type = EAssetReferenceType::None;

	TWeakObjectPtr<const UClass> PropertyClass = nullptr;
	bool bIsIndirectReference = false;
};
using FReferencingPropertyDescriptionPtr = TSharedPtr<FReferencingPropertyDescription>;

/**
 * A node to display a list of Node properties which are referencing another Node/Asset
 */
UCLASS(MinimalAPI)
class UEdGraphNode_ReferencedProperties : public UEdGraphNode
{
	GENERATED_BODY()

public:
	const TArray<FReferencingPropertyDescriptionPtr>& GetReferencedPropertiesDescription() const
	{
		return ReferencedPropertyDescription;
	}

	const TObjectPtr<UEdGraphNode_Reference>& GetReferencingNode() const { return ReferencingNode; }

	const TObjectPtr<UEdGraphNode_Reference>& GetReferencedNode() const { return ReferencedNode; }

	UE_API UObject* GetReferencingObject() const;
	
	UE_API UObject* GetReferencedObject() const;

	/** Initialize this Node */
	UE_API void SetupReferencedPropertiesNode(const TArray<FReferencingPropertyDescription>& InPropertiesDescription
		, const TObjectPtr<UEdGraphNode_Reference>& InReferencingNode, const TObjectPtr<UEdGraphNode_Reference>& InReferencedNode);

	/**
	 * Refresh the node location, so it stays mid-way between Referencing and Referenced nodes
	 * @param InNodeSize: size of the widget representing this node. Useful when centering
	 */
	UE_API void RefreshLocation(const FVector2f& InNodeSize = FVector2f::Zero());

	DECLARE_MULTICAST_DELEGATE(FOnPropertiesDescriptionUpdated)
	FOnPropertiesDescriptionUpdated& OnPropertiesDescriptionUpdated() { return OnPropertiesDescriptionUpdatedDelegate; }

private:
	TArray<FReferencingPropertyDescriptionPtr> ReferencedPropertyDescription;

	UPROPERTY()
	TObjectPtr<UEdGraphNode_Reference> ReferencingNode;

	UPROPERTY()
	TObjectPtr<UEdGraphNode_Reference> ReferencedNode;

	FOnPropertiesDescriptionUpdated OnPropertiesDescriptionUpdatedDelegate;
};

#undef UE_API
