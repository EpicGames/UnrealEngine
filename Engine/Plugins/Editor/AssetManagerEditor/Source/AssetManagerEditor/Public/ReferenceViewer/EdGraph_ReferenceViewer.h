// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "AssetRegistry/AssetData.h"
#include "AssetManagerEditorModule.h"
#include "EdGraph/EdGraph.h"
#include "Misc/AssetRegistryInterface.h"
#include "Misc/FilterCollection.h"
#include "EdGraph_ReferenceViewer.generated.h"

#define UE_API ASSETMANAGEREDITOR_API

class FAssetThumbnailPool;
class UEdGraphNode_Reference;
class UEdGraphNode_ReferencedProperties;
class SReferenceViewer;
class UReferenceViewerSettings;
enum class EDependencyPinCategory;
struct FReferencingPropertyDescription;

/*
*  Holds asset information for building reference graph
*/ 
struct FReferenceNodeInfo
{
	FAssetIdentifier AssetId;

	FAssetData AssetData;

	// immediate children (references or dependencies)
	TArray<TPair<FAssetIdentifier, EDependencyPinCategory>> Children;

	// this node's parent references (how it got included)
	TArray<FAssetIdentifier> Parents;

	// Which direction.  Referencers are left (other assets that depend on me), Dependencies are right (other assets I depend on)
	bool bReferencers;

	bool bIsRedirector;

	int32 OverflowCount;

	// Denote when all children have been manually expanded and the breadth limit should be ignored
	bool bExpandAllChildren;

	FReferenceNodeInfo(const FAssetIdentifier& InAssetId, bool InbReferencers);

	bool IsFirstParent(const FAssetIdentifier& InParentId) const;
	
	bool IsRedirector() const;

	bool IsADuplicate() const;

	// The Provision Size, or vertical spacing required for layout, for a given parent.  
	// At the time of writing, the intent is only the first node manifestation of 
	// an asset will have its children shown
	int32 ProvisionSize(const FAssetIdentifier& InParentId) const;

	// how many nodes worth of children require vertical spacing 
	int32 ChildProvisionSize;

	// Whether or not this nodeinfo passed the current filters 
	bool PassedFilters;

};



UCLASS(MinimalAPI)
class UEdGraph_ReferenceViewer : public UEdGraph
{
	GENERATED_UCLASS_BODY()

public:
	// UObject implementation
	UE_API virtual void BeginDestroy() override;
	// End UObject implementation

	/** Set reference viewer to focus on these assets */
	UE_API void SetGraphRoot(const TArray<FAssetIdentifier>& GraphRootIdentifiers, const FIntPoint& GraphRootOrigin = FIntPoint(ForceInitToZero));

	/** Returns list of currently focused assets */
	UE_API const TArray<FAssetIdentifier>& GetCurrentGraphRootIdentifiers() const;

	/** If you're extending the reference viewer via GetAllGraphEditorContextMenuExtender you can use this to get the list of selected assets to use in your menu extender */
	UE_API bool GetSelectedAssetsForMenuExtender(const class UEdGraphNode* Node, TArray<FAssetIdentifier>& SelectedAssets) const;

	/** Accessor for the thumbnail pool in this graph */
	UE_API const TSharedPtr<class FAssetThumbnailPool>& GetAssetThumbnailPool() const;

	/** Force the graph to rebuild */
	UE_API class UEdGraphNode_Reference* RebuildGraph();

	/** Refilters the nodes, more efficient that a full rebuild.  This function is preferred when the assets, reference types or depth hasn't changed, meaning the NodeInfos didn't change, just 
	 * the presentation or filtering */
	UE_API class UEdGraphNode_Reference* RefilterGraph();

	UE_INTERNAL bool IsShowingContentVersePath() const { return bShowingContentVersePath; }
	UE_INTERNAL UE_API void SetShowingContentVersePath(bool bInShowingContentVersePath);
	UE_INTERNAL UE_API void UpdatePaths();

	using FIsAssetIdentifierPassingSearchFilterCallback = TFunction<bool(const FAssetIdentifier&)>;
	UE_DEPRECATED(5.7, "Call SetDoesAssetPassSearchFilterCallback instead.")
	UE_API void SetIsAssetIdentifierPassingSearchFilterCallback(const TOptional<FIsAssetIdentifierPassingSearchFilterCallback>& InIsAssetIdentifierPassingSearchFilterCallback);

	using FDoesAssetPassSearchFilterCallback = TFunction<bool(const FAssetIdentifier&, const FAssetData&)>;
	void SetDoesAssetPassSearchFilterCallback(const FDoesAssetPassSearchFilterCallback& InDoesAssetPassSearchFilterCallback) { DoesAssetPassSearchFilterCallback = InDoesAssetPassSearchFilterCallback; }

	UE_DEPRECATED(5.6, "Use the ICollectionContainer overload instead.")
	UE_API FName GetCurrentCollectionFilter() const;
	UE_API void GetCurrentCollectionFilter(ICollectionContainer*& OutCollectionContainer, FName& OutCollectionName) const;

	UE_DEPRECATED(5.6, "Use the ICollectionContainer overload instead.")
	UE_API void SetCurrentCollectionFilter(FName NewFilter);
	UE_API void SetCurrentCollectionFilter(const TSharedPtr<ICollectionContainer>& CollectionContainer, FName CollectionName);

	UE_API TArray<FName> GetCurrentPluginFilter() const;
	UE_API void SetCurrentPluginFilter(TArray<FName> NewFilter);
	UE_API TArray<FName> GetEncounteredPluginsAmongNodes() const;

	/* Delegate type to notify when the assets or NodeInfos have changed as opposed to when the filters changed */
	FSimpleMulticastDelegate& OnAssetsChanged() { return OnAssetsChangedDelegate; }

	/* Not to be confused with the above Content Browser Collection name, this is a TFiltercollection, a list of active filters */
	UE_API void SetCurrentFilterCollection(TSharedPtr< TFilterCollection<FReferenceNodeInfo&> > NewFilterCollection);

	/* Returns a set of unique asset types as UClass* */
	const TSet<FTopLevelAssetPath>& GetAssetTypes() const { return CurrentClasses; }

	/* Returns true if the current graph has overflow nodes */
	bool BreadthLimitExceeded() const { return bBreadthLimitReached; };

	/** Refreshes the information of existing Referenced Properties Nodes */
	UE_API void RefreshReferencedPropertiesNodes();

	/** Closes (removes) the specified Referenced Properties Node */
	UE_API void CloseReferencedPropertiesNode(UEdGraphNode_ReferencedProperties* InNode);

private:

	/**
	 * Retrieves the list of properties/values of a specified Referencer Object which reference the specified Referenced Asset
	 * @param InReferencer: the Object referencing the Asset
	 * @param InReferencedAsset: the asset referenced by the specified Object
	 * @return the list of properties of the input Referencer, referencing the specified Asset.
	 */
	UE_API TArray<FReferencingPropertyDescription> RetrieveReferencingProperties(UObject* InReferencer, UObject* InReferencedAsset);

	UE_API TWeakPtr<SReferenceViewer> GetReferenceViewer() const;
	UE_API void SetReferenceViewer(TSharedPtr<SReferenceViewer> InViewer);
	UE_API UEdGraphNode_Reference* ConstructNodes(const TArray<FAssetIdentifier>& GraphRootIdentifiers, const FIntPoint& GraphRootOrigin);


	UE_API bool ExceedsMaxSearchDepth(int32 Depth, int32 MaxDepth) const;
	UE_API bool ExceedsMaxSearchBreadth(int32 Breadth) const;
	UE_API FAssetManagerDependencyQuery GetReferenceSearchFlags(bool bHardOnly) const;

	UE_API UEdGraphNode_Reference* CreateReferenceNode();

	UE_API UEdGraphNode_ReferencedProperties* CreateReferencedPropertiesNode(const TArray<FReferencingPropertyDescription>& InPropertiesDescriptionArray
	, const TObjectPtr<UEdGraphNode_Reference>& InReferencingNode, const TObjectPtr<UEdGraphNode_Reference>& InReferencedNode);

	/* Generates a NodeInfo structure then used to generate and layout the graph nodes */
	UE_API void RecursivelyPopulateNodeInfos(bool bReferencers, const TArray<FAssetIdentifier>& Identifiers, TMap<FAssetIdentifier, FReferenceNodeInfo>& NodeInfos, int32 CurrentDepth, int32 MaxDepth);

	/* Marks up the NodeInfos with updated filter information and provision sizes */
	UE_API void RecursivelyFilterNodeInfos(const FAssetIdentifier& InAssetId, TMap<FAssetIdentifier, FReferenceNodeInfo>& NodeInfos, int32 CurrentDepth, int32 MaxDepth);

	/* Searches for the AssetData for the list of packages derived from the AssetReferences  */
	UE_API void GatherAssetData(TMap<FAssetIdentifier, FReferenceNodeInfo>& InNodeInfos);

	/* Uses the NodeInfos map to generate and layout the graph nodes */
	UE_API UEdGraphNode_Reference* RecursivelyCreateNodes(
		bool bInReferencers, 
		const FAssetIdentifier& InAssetId, 
		const FIntPoint& InNodeLoc, 
		const FAssetIdentifier& InParentId, 
		UEdGraphNode_Reference* InParentNode, 
		TMap<FAssetIdentifier, FReferenceNodeInfo>& InNodeInfos, 
		int32 InCurrentDepth, 
		int32 InMaxDepth, 
		bool bIsRoot = false
	);

	UE_API void ExpandNode(bool bReferencers, const FAssetIdentifier& InAssetIdentifier);

	/** Removes all nodes from the graph */
	UE_API void RemoveAllNodes();

	/** Returns true if filtering is enabled and we have a valid collection */
	UE_API bool ShouldFilterByCollection() const;
	
	/** Returns true if filtering is enabled and we have a valid plugin name filter set */
	UE_API bool ShouldFilterByPlugin() const;
	
	UE_API void GetUnfilteredGraphPluginNamesRecursive(bool bReferencers, const FAssetIdentifier& InAssetIdentifier, int32 InCurrentDepth, int32 InMaxDepth, const FAssetManagerDependencyQuery& Query, TSet<FAssetIdentifier>& OutAssetIdentifiers);
	UE_API void GetUnfilteredGraphPluginNames(TArray<FAssetIdentifier> RootIdentifiers, TArray<FName>& OutPluginNames);

	UE_API void GetSortedLinks(const TArray<FAssetIdentifier>& Identifiers, bool bReferencers, const FAssetManagerDependencyQuery& Query, TMap<FAssetIdentifier, EDependencyPinCategory>& OutLinks) const;
	UE_API bool IsPackageIdentifierPassingFilter(const FAssetIdentifier& InAssetIdentifier) const;
	UE_API bool IsPackageIdentifierPassingPluginFilter(const FAssetIdentifier& InAssetIdentifier) const;
	UE_API bool DoesAssetPassSearchTextFilter(const FAssetIdentifier& InAssetIdentifier, const FAssetData& InAssetData) const;

	UE_API UEdGraphNode_Reference* FindPath(const FAssetIdentifier& RootId, const FAssetIdentifier& TargetId);
	UE_API bool FindPath_Recursive(bool bInReferencers, const FAssetIdentifier& InAssetId, const FAssetIdentifier& Target, TMap<FAssetIdentifier, FReferenceNodeInfo>& InNodeInfos, TSet<FAssetIdentifier>& Visited);

	UE_API void RefreshReferencedPropertiesNode(const UEdGraphNode_ReferencedProperties* InNode);
private:
	/** Pool for maintaining and rendering thumbnails */
	TSharedPtr<FAssetThumbnailPool> AssetThumbnailPool;

	/** Editor for this pool */
	TWeakPtr<SReferenceViewer> ReferenceViewer;

	TArray<FAssetIdentifier> CurrentGraphRootIdentifiers;
	FIntPoint CurrentGraphRootOrigin;

	/** Stores if the breadth limit was reached on the last refilter*/
	bool bBreadthLimitReached;

	/** Whether to show Verse paths */
	bool bShowingContentVersePath;

	/** Current collection filter. NAME_None for no filter */
	TSharedPtr<ICollectionContainer> CurrentCollectionFilterContainer;
	FName CurrentCollectionFilterName;

	/** Current plugin filter. Empty for no filter. */
	TArray<FName> CurrentPluginFilter;

	/** Plugin names found among unfiltered nodes. Chose among these when filtering for plugins. */
	TArray<FName> EncounteredPluginsAmongNodes;

	/** A set of the unique class types referenced */
	TSet<FTopLevelAssetPath> CurrentClasses;

	/* Cached Reference Information used to quickly refilter */
	TMap<FAssetIdentifier, FReferenceNodeInfo> ReferencerNodeInfos;
	TMap<FAssetIdentifier, FReferenceNodeInfo> DependencyNodeInfos;

	FDoesAssetPassSearchFilterCallback DoesAssetPassSearchFilterCallback;

	/** List of packages the current collection filter allows */
	TSet<FName> CurrentCollectionPackages;

	/** Current filter collection */
	TSharedPtr< TFilterCollection<FReferenceNodeInfo & > > FilterCollection;

	UReferenceViewerSettings* Settings;

	/* A delegate to notify when the underlying assets changed (usually through a root or depth change) */
	FSimpleMulticastDelegate OnAssetsChangedDelegate;

	FAssetIdentifier TargetIdentifier;

	/** Keeping track of existing Referencing Properties Nodes */
	TMap<uint32, TWeakObjectPtr<UEdGraphNode_ReferencedProperties>> ReferencedPropertiesNodes;

	friend SReferenceViewer;
	friend class UReferenceViewerSchema;
};

#undef UE_API
