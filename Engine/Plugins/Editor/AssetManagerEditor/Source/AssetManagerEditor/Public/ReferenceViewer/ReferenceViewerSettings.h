// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"

#include "ReferenceViewerSettings.generated.h"

/**
 *  Project based Reference Viewer Saved Settings 
 */
USTRUCT()
struct FilterState
{
	GENERATED_BODY();

	FilterState()
	: FilterPath()
	, bIsEnabled(false)
	{}

	FilterState(FTopLevelAssetPath InFilterPath, bool InState)
	: FilterPath(InFilterPath)
	, bIsEnabled(InState)
	{}

	UPROPERTY()
	FTopLevelAssetPath FilterPath;

	UPROPERTY()
	bool bIsEnabled;
};

/** @see tooltips in FAssetManagerEditorCommands::RegisterCommands. */
UENUM()
enum class EEditorOnlyReferenceFilterType
{
	Game,
	Propagation,
	EditorOnly,
};

UCLASS(config=EditorPerProjectUserSettings)
class UReferenceViewerSettings : public UObject
{
public:
	GENERATED_BODY()

	bool IsSearchDepthLimited() const;
	void SetSearchDepthLimitEnabled(bool bNewEnabled);

	bool IsShowReferencers() const;
	void SetShowReferencers(const bool bShouldShowReferencers);

	int32 GetSearchReferencerDepthLimit() const;
	void SetSearchReferencerDepthLimit(int32 NewDepthLimit, bool bSaveConfig = true);

	bool IsShowDependencies() const;
	void SetShowDependencies(const bool bShouldShowDependencies);

	int32 GetSearchDependencyDepthLimit() const;
	void SetSearchDependencyDepthLimit(int32 NewDepthLimit, bool bSaveConfig = true);

	UE_DEPRECATED(5.5, "Search breadth is always limited. Will always return true.")
	bool IsSearchBreadthLimited() const { return true; }

	UE_DEPRECATED(5.5, "Search breadth is always limited.")
	void SetSearchBreadthLimitEnabled(bool bNewEnabled) {}

	int32 GetSearchBreadthLimit() const;
	void SetSearchBreadthLimit(int32 NewBreadthLimit);

	bool GetEnableCollectionFilter() const;
	void SetEnableCollectionFilter(bool bEnabled);

	bool GetEnablePluginFilter() const;
	void SetEnablePluginFilter(bool bEnabled);

	bool IsShowSoftReferences() const;
	void SetShowSoftReferencesEnabled(bool bNewEnabled);

	bool IsShowHardReferences() const;
	void SetShowHardReferencesEnabled(bool bNewEnabled);

	UE_DEPRECATED(5.5, "Use GetEditorOnlyReferenceFilterType.")
	bool IsShowEditorOnlyReferences() const { return true; }
	UE_DEPRECATED(5.5, "Use SetEditorOnlyReferenceFilterType.")
	void SetShowEditorOnlyReferencesEnabled(bool bNewEnabled) {}

	EEditorOnlyReferenceFilterType GetEditorOnlyReferenceFilterType() const;
	void SetEditorOnlyReferenceFilterType(EEditorOnlyReferenceFilterType Value);

	bool IsShowManagementReferences() const;
	void SetShowManagementReferencesEnabled(bool bNewEnabled);

	bool IsShowSearchableNames() const;
	void SetShowSearchableNames(bool bNewEnabled);

	bool IsShowCodePackages() const;
	void SetShowCodePackages(bool bNewEnabled);

	bool IsShowDuplicates() const;
	void SetShowDuplicatesEnabled(bool bNewEnabled);

	bool IsShowFilteredPackagesOnly() const;
	void SetShowFilteredPackagesOnlyEnabled(bool bNewEnabled);

	bool IsCompactMode() const;
	void SetCompactModeEnabled(bool bNewEnabled);

	bool IsShowExternalReferencers() const;
	void SetShowExternalReferencersEnabled(bool bNewEnabled);

	bool IsShowPath() const;
	void SetShowPathEnabled(bool bNewEnabled);

	bool GetFiltersEnabled() const;
	void SetFiltersEnabled(bool bNewEnabled);

	bool AutoUpdateFilters() const;
	void SetAutoUpdateFilters(bool bNewEnabled);

	const TArray<FilterState>& GetUserFilters() const;
	void SetUserFilters(TArray<FilterState>& InFilters);

	const TMap<FString, bool>& GetCustomFilters() const;
	void SetCustomFilters(const TMap<FString, bool>& InCustomFilters);

	bool GetFindPathEnabled() const;
	void SetFindPathEnabled(bool bNewEnabled);

private:
	/* Whether to limit the search depth for Referencers & Dependencies */
	UPROPERTY(config)
	bool bLimitSearchDepth;
	
	/* Whether to display the Referencers */
	UPROPERTY(config)
	bool bIsShowReferencers;
	
	/* How deep to search references */
	UPROPERTY(config)
	int32 MaxSearchReferencerDepth; 
	
	/* Whether to display the Dependencies */
	UPROPERTY(config)
	bool bIsShowDependencies;
	
	/* How deep to search dependanies */
	UPROPERTY(config)
	int32 MaxSearchDependencyDepth; 
	
	/* Whether or not to limit how many siblings can appear */
	UE_DEPRECATED(5.5, "bLimitSearchBreadth has been deprecated.")
	UPROPERTY(config, meta = (DeprecatedProperty, DeprecationMessage = "Search breadth is always limited."))
	bool bLimitSearchBreadth_DEPRECATED;
	
	/* The max number of siblings that can appear from a node */
	UPROPERTY(config)
	int32 MaxSearchBreadth;
	
	/* Whether or not to filter from a collection */
	UPROPERTY(config)
	bool bEnableCollectionFilter;
	
	/* Whether or not to filter for assets by plugins. */
	UPROPERTY(config)
	bool bEnablePluginFilter;
	
	/* Show/Hide Soft References */
	UPROPERTY(config)
	bool bIsShowSoftReferences;
	
	/* Show/Hide Hard References */
	UPROPERTY(config)
	bool bIsShowHardReferences;
	
	UE_DEPRECATED(5.5, "Use EditorOnlyReferenceFilterType.")
	UPROPERTY(config, meta = (DeprecatedProperty, DeprecationMessage = "Use EditorOnlyReferenceFilterType."))
	bool bIsShowEditorOnlyReferences;

	UPROPERTY(config)
	EEditorOnlyReferenceFilterType EditorOnlyReferenceFilterType;

	/* Show/Hide Management Assets (i.e. PrimaryAssetIds) */
	UPROPERTY(config)
	bool bIsShowManagementReferences;
	
	/* Show/Hide Searchable Names (i.e. Gameplay Tags) */
	UPROPERTY(config)
	bool bIsShowSearchableNames;
	
	/* Show/Hide Native Packages  */
	UPROPERTY(config)
	bool bIsShowCodePackages;
	
	/* Whether to show duplicate asset references */
	UPROPERTY(config)
	bool bIsShowDuplicates;
	
	/* Whether to filter the search results or just select them  */
	UPROPERTY(config)
	bool bIsShowFilteredPackagesOnly;
	
	/* Whether to show the nodes in a compact (no thumbnail) view */
	UPROPERTY(config)
	bool bIsCompactMode;

	/* Whether to show external referencers or collapsed owner asset */
	UPROPERTY(config)
	bool bIsShowExternalReferencers;

	/* Whether to show the package's path as a comment */
	UPROPERTY(config)
	bool bIsShowPath;

	/* This turns on/off any filtering done though the SFilterBar */
	UPROPERTY(config)
	bool bFiltersEnabled;

	/* When true, the filters bar auto updates based on the node types, otherwise user filters will be used */
	UPROPERTY(config)
	bool bAutoUpdateFilters;

	/* The list of filters the user has built up */
	UPROPERTY(config)
	TArray<FilterState> UserFilters;

	/* The list of reference viewer specific filters the user has built up */
	UPROPERTY(config)
	TMap<FString, bool> CustomFilters;

	/* Enable the Find Path tool */
	UPROPERTY(config)
	bool bFindPathEnabled;

};
